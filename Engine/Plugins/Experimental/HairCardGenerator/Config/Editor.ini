[/Script/HairCardGeneratorEditor.HairCardGeneratorEditorSettings]
HairCardAssetNameFormat = "{groom_name}_LOD{lod}"
HairCardAssetPathFormat = "{groom_dir}/{groom_name}_LOD{lod}"
