{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Hair Card Generator",
	"Description": "Procedurally generate hair cards from hair strands",
	"Category": "Geometry",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "https://forums.unrealengine.com/",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "HairCardGeneratorEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		},
		{
			"Name": "HairCardGeneratorDataflow",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "HairStrands",
			"Enabled": true
		},
		{
			"Name": "PythonMLPackages",
			"Enabled": true
		},
		{
			"Name": "Dataflow",
			"Enabled": true
		}
	],
	"PythonRequirements": [
		{
			"Platform": "All",
			"Requirements": [
				"addict==2.4.0 --hash=sha256:249bb56bbfd3cdc2a004ea0ff4c2b6ddc84d53bc2194761636eb314d5cfa5dfc",
				"glcontext==3.0.0 --hash=sha256:04921720740438ceea8fb8a38b5665963520c7c8f27bef03df8aeb3ea3cfbfb6 --hash=sha256:12c2abef8efabb8ab7e35d16785968de888ae7349d1b83c765080f35fcd3c6e5 --hash=sha256:1445a03d8795113034e1f9ffa662f795df65ae69ee21b26ed3b1d66100ba3f8c --hash=sha256:3a9e56fa3597cc709cfd0fdf2ae682cda36510a13faac2b3142f401e823b64f4 --hash=sha256:4817f4cd52c7fe5410c92ca12b6712435548918719373882ade76f8f75d80abd --hash=sha256:4a2972f92da9a6fb06860e117de05f5b8adc2e2d827bbc0ccc7acbe7325acd1e --hash=sha256:4b3b12a66f57379566dd4d36899ac265abdbe040f3fc3293f50cd6678a1dcc9b --hash=sha256:7043f59d126feb26896a6419ebfeecc78c07ffefced2a2f59104dd7a2f71ebb5 --hash=sha256:7fa413f4420abff2bbb5aa5770a3e1deffcdc13e0ef2f459b145fa79c36909e7 --hash=sha256:848f870a2bc72a29de7ab6756b9e8f2e6ce052e17873ebc6b3f25129b6e0d58a --hash=sha256:a0484308af75e04b0e56066dc2324a8fb9f1443b76ddb98833439982322b2a39 --hash=sha256:b154c25a57e16dbb073478d0cbe2c0090649d135c4c9f87e753b6181b97ec848 --hash=sha256:cdd81e8b580e43c1fe1c48c0fc3909e6c1f37ee4cfd8c990c03b2df3b463bd37 --hash=sha256:e80bb37ba727bd20c192f2754aea40c437a7665005c1001c10752f91913964e9 --hash=sha256:ef4b4ec35e2b720f4cd250bb92cf6417add445490bf780345596da5a796a0e6f --hash=sha256:fa5a14778d13ecf4a0dd60a7825427bf6ac0383eacb3b8b1a9c23e4976aa0c8c",
				"joblib==1.2.0 --hash=sha256:091138ed78f800342968c523bdde947e7a305b8594b910a0fea2ab83c3c6d385",
				"moderngl==5.12.0 --hash=sha256:0c210e8d52a60025f6586ca015c39feb1e57e6dc792c3ff44800f6493a541b1a --hash=sha256:105cdee2ee29a7e6ebbc76cf178941503656a185c6945933bc7ef395ba8e65a7 --hash=sha256:18deb8bebd0a4277d92c76dbedf8e4b4b68bf0a8a878404c6b26aed750890d3b --hash=sha256:2750547707c1ec3790dfbeb9c90fb808672ff13f61cac392c706ba09fda10db0 --hash=sha256:28b9eb3574ffc6e303173ca0a419b63d8b12cd67f924289c02d127c4d17cdca5 --hash=sha256:28cdba5dcf2d03c89bb25dc3b2f5770ac4104470ed5bbe680a15494fa52a537d --hash=sha256:49a6e27abafacef104c7ca336f6790f91c69617a1d752ead4a017b706d632b55 --hash=sha256:7146c6fec3b2d7e8d11fa2504046b186d396520c0c2f7ef3aed40d8456dbebfc --hash=sha256:76d966194d51852f48c42a6e786a4520f1e1be5f93e2626423d673663422d559 --hash=sha256:7fc0f8788bc84433d2124e9a4893adbe40f93c7d213abb8ad7b909540cb0161f --hash=sha256:8698a59ad03539a2982125b7998efc1c107ba31d5d03437b6fcd72cb2c226922 --hash=sha256:8f0c4f7c42425177168938386a4fabd734ca3bbb5de2d1fd1176cfa6f980fc13 --hash=sha256:b6c4972f3ddd10a3de6c30311da2c25bc493d023796e16c5d4e0f8bd6d5770be --hash=sha256:c5c2a5fe06c7021183d9274df798f25516409c8d55898c324dae8a0b2de10144 --hash=sha256:dad93893e3fcb2410bfd31e854f20e1370b4fbafa07a737f1046f5fbd29ba0f4 --hash=sha256:e801cd0d35b4e3e99fc6a6f15eb193ce907bfa78127afa5825f1fad24c700c0e",
				"multipledispatch==0.6.0 --hash=sha256:a55c512128fb3f7c2efd2533f2550accb93c35f1045242ef74645fc92a2c3cba",
				"PyOpenGL==3.1.6 --hash=sha256:a7139bc3e15d656feae1f7e3ef68c799941ed43fadc78177a23db7e946c20738",
				"pyrr==0.10.3 --hash=sha256:d8af23fb9bb29262405845e1c98f7339fbba5e49323b98528bd01160a75c65ac",
				"PyYAML==6.0.2 --hash=sha256:1e2120ef853f59c7419231f3bf4e7021f1b936f6ebd222406c3b60212205d2ee --hash=sha256:3ad2a3decf9aaba3d29c8f537ac4b243e36bef957511b4766cb0057d32b0be85 --hash=sha256:7e7401d0de89a9a855c839bc697c079a4af81cf878373abd7dc625847d25cbd8 --hash=sha256:80bab7bfc629882493af4aa31a4cfa43a4c57c83813253626916b8c7ada83476 --hash=sha256:c70c95198c015b85feafc136515252a261a84561b7b1d51e3384e0655ddf25ab --hash=sha256:cc1c1159b3d456576af7a3e4d1ba7e6924cb39de8f67111c735f6fc832082774 --hash=sha256:ce826d6ef20b1bc864f0a68340c8b3287705cae2f8b4b1d932177dcc76721725 --hash=sha256:e10ce637b18caea04431ce14fabcf5c64a1c61ec9c56b071a4b7ca131ca52d44",
				"scikit_learn==1.2.1 --hash=sha256:479aedd0abedbda6b8b4529145fe4cd8622f69f726a72cef8f75548a93eeb1e1 --hash=sha256:5523e21ab2b4d52b2bd41bedd335dbe8f3c1b5f6dd7c9c001b2e17ec9818af8d --hash=sha256:5a8111f3c7a314017ebf90d6feab861c11d1ca14f3dbafb39abcc31aa4c54ba6 --hash=sha256:61bb9c654b5d2e6cdd4b1c7e6048fc66270c1682bda1b0f7d2726fdae09010f4 --hash=sha256:70fa30d146b7e9d0c256e73e271b3e17f23123b7c4adcbde1a385031adf59090 --hash=sha256:a9abf17d177df54e529154f26acfd42930e19117d045e8a9a8e893ca82dd94ec --hash=sha256:bed9f75763bd392c094bf474c7ab75a01d68b15146ea7a20c0f9ff6fb3063dad --hash=sha256:c722f3446ad8c4f1a93b2399fe1a188635b94709a3f25e6f4d61efbe75fe8eaa --hash=sha256:c9285275a435d1f8f47bbe3500346ab9ead2499e0e090518404d318ea90d1c1c --hash=sha256:cba0c7c6bf1493f8ce670bab69f9317874826ee838988de377ae355abd4d74cf --hash=sha256:d00e46a2a7fce6e118ed0f4c6263785bf6c297a94ffd0cd7b32455043c508cc8 --hash=sha256:d8bcd303dd982494842a3f482f844d539484c6043b4eed896b43ea8e5f609a21 --hash=sha256:da0e2d50a8435ea8dc5cd21f1fc1a45d329bae03dcca92087ebed859d22d184e --hash=sha256:dc838b5a4057c55ba81b82316ea8bf443af445f96eb21500b0e40618017e0923 --hash=sha256:e0ee4d4d32c94e082344308528f7b3c9294b60ab19c84eb37a2d9c88bdffd9d1",
				"six==1.16.0 --hash=sha256:8abb2f1d86890a2dfb989f9a77cfcfd3e47c2a354b01111771326f8aa26e0254",
				"threadpoolctl==3.1.0 --hash=sha256:8b99adda265feb6773280df41eece7b2e6561b772d21ffd52e372f999024907b"
			]
		}
	]
}