// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class HairCardGeneratorDataflow: ModuleRules
	{
		public HairCardGeneratorDataflow(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePaths.AddRange(new string[] {
				System.IO.Path.Combine(GetModuleDirectory("HairCardGeneratorEditor"), "Private"),
			});
			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"ChaosCore",
					"Chaos",
					"DataflowCore",
					"DataflowEngine",
					"DataflowEditor",
					"DataflowEnginePlugin",
					"HairStrandsCore",
					"HairStrandsDataflow",
					"HairStrandsEditor",
					"HairCardGeneratorEditor",
					"HairCardGeneratorFramework",
					"Engine",
					"GeometryCore",
					"PlanarCut",
					"GeometryFramework"
				}
			);
		}
	}
}