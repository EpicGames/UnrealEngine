// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildCardsSettingsNode.h"

#include "GroomBuilder.h"
#include "HairCardGeneratorEditorModule.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "GeometryCollection/Facades/CollectionConstraintOverrideFacade.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BuildCardsSettingsNode)

namespace UE::CardGen::Private
{
	static void BuildStrandsPositions(const FManagedArrayCollection& GroomCollection, TArray<TArray<FVector>>& StrandsPositions, const FHairDescription& HairDescription)
	{
		const GeometryCollection::Facades::FCollectionCurveGeometryFacade CurvesFacade(GroomCollection);
		if (CurvesFacade.IsValid())
		{
			FHairDescriptionGroups HairDescriptionGroups;
			FGroomBuilder::BuildHairDescriptionGroups(HairDescription, HairDescriptionGroups, false);
			
			StrandsPositions.SetNum(HairDescription.GetNumStrands());

			int32 PointOffset = 0;
			const int32 NumCurves = CurvesFacade.GetNumCurves();
			for(int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
			{
				const int32 GroupIndex = CurvesFacade.GetCurveGeometryIndices()[CurveIndex];
				const int32 SourceCurve = CurvesFacade.GetCurveSourceIndices()[CurveIndex] >> 1;
				
				if (HairDescriptionGroups.HairGroups.IsValidIndex(GroupIndex) && HairDescriptionGroups.HairGroups[GroupIndex].Strands.StrandsCurves.CurvesMapping.IsValidIndex(SourceCurve))
				{
					const int32 StrandID = HairDescriptionGroups.HairGroups[GroupIndex].Strands.StrandsCurves.CurvesMapping[SourceCurve];
				
					const int32 NumPoints = CurvesFacade.GetCurvePointOffsets()[CurveIndex]-PointOffset;
					if (StrandsPositions.IsValidIndex(StrandID))
					{
						StrandsPositions[StrandID].SetNum(NumPoints);
				
						for(int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
						{
							StrandsPositions[StrandID][PointIndex] = FVector(CurvesFacade.GetPointRestPositions()[PointIndex + PointOffset]);
						}
					}
				}
				PointOffset = CurvesFacade.GetCurvePointOffsets()[CurveIndex];
			}
		}
	}
	
	static FString DisplayCardsGroups(const UGroomAsset* GroomAsset)
	{
		TArray<FName> GroupNames;
		TArray<int32> GroupCount;
		FHairCardGeneratorUtils::BuildCardsGroups(GroomAsset, GroupNames, GroupCount);

		FString CardsGroups;
		if(GroupNames.Num() == GroupCount.Num())
		{
			for(int32 GroupIndex = 0; GroupIndex < GroupNames.Num(); ++GroupIndex)
			{
				CardsGroups += FString("Cards Groups Name = ") + GroupNames[GroupIndex].ToString() + FString(" | Num Strands = ") +
					FString::FromInt(GroupCount[GroupIndex]) + FString("\n");
			}
		}
		return CardsGroups;
	}
}

FBuildCardsSettingsNode::FBuildCardsSettingsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&GroomAsset);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&CardsSettings);
}

void FBuildCardsSettingsNode::OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FBuildCardsSettingsNode, GenerationSettings) || 
		InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FBuildCardsSettingsNode, GroomAsset))
	{
		if (UGroomAsset* LocalAsset = GetValue<TObjectPtr<UGroomAsset>>(Context, &GroomAsset))
		{
			if (AssetsDirectoryPath.IsEmpty())
			{
				AssetsDirectoryPath = HairCardSettings_Helpers::GetGroomDirectoryForHairCards(LocalAsset);
			}
			if (AssetsBaseName.IsEmpty())
			{
				AssetsBaseName = HairCardSettings_Helpers::GetGroomNameForHairCards(LocalAsset);
			}
		}
	}
}

void FBuildCardsSettingsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection GeometryCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		SetValue(Context, MoveTemp(GeometryCollection), &Collection);
	}
	else if (Out->IsA<TArray<FGroomCardsSettings>>(&CardsSettings))
	{
		const FManagedArrayCollection& GeometryCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		UGroomAsset* LocalAsset = GetValue<TObjectPtr<UGroomAsset>>(Context, &GroomAsset);
		TArray<FGroomCardsSettings> OutputSettings;
		if (LocalAsset)
		{
			for(const FGroomGenerationSettings& LocalSettings : GenerationSettings)
			{
				FHairGroupsCardsSourceDescription* CardsDescriptionPtr = nullptr;
				
				bool bHasCardDescription = false;
				for(FHairGroupsCardsSourceDescription& HairCardsDescription : LocalAsset->GetHairGroupsCards())
				{
					if((LocalSettings.GroupIndex == HairCardsDescription.GroupIndex) &&
								(LocalSettings.LODIndex == HairCardsDescription.LODIndex) && (HairCardsDescription.LODIndex != INDEX_NONE))
					{
						bHasCardDescription = true;
						CardsDescriptionPtr = &HairCardsDescription;
						break;
					}
				}
				if (!bHasCardDescription)
				{
					FHairGroupsCardsSourceDescription CardsDescription;
					CardsDescription.LODIndex = LocalSettings.LODIndex;
					CardsDescription.GroupIndex = LocalSettings.GroupIndex;
					LocalAsset->GetHairGroupsCards().Add(CardsDescription);

					FHairGroupsCardsSourceDescription& HairCardsDescription = LocalAsset->GetHairGroupsCards().Last();
					CardsDescriptionPtr = &HairCardsDescription;
				}

				if (CardsDescriptionPtr)
				{
					CardsDescriptionPtr->Textures = FHairGroupCardsTextures();

					FCardsGenerationAdvancedOptions AdvancedOptions;
					AdvancedOptions.bGenerateGeometryForAllGroups = LocalSettings.AdvancedOptions.bGenerateGeometryForAllGroups;
					AdvancedOptions.bReduceCardsFromPreviousLOD = LocalSettings.AdvancedOptions.bReduceCardsFromPreviousLOD;
					AdvancedOptions.RandomSeed = LocalSettings.AdvancedOptions.RandomSeed;
					AdvancedOptions.bUseReservedSpaceFromPreviousLOD = LocalSettings.AdvancedOptions.bUseReservedSpaceFromPreviousLOD;
					AdvancedOptions.AtlasSize = static_cast<uint8>(LocalSettings.AdvancedOptions.AtlasSize);
					AdvancedOptions.ReserveTextureSpaceLOD = LocalSettings.AdvancedOptions.ReserveTextureSpaceLOD;
					AdvancedOptions.bUseGroomAssetStrandWidth = LocalSettings.AdvancedOptions.bUseGroomAssetStrandWidth;
					AdvancedOptions.DefaultDirectory = AssetsDirectoryPath;
					AdvancedOptions.DefaultName= AssetsBaseName;
				
					OutputSettings.AddDefaulted();
					FHairCardGeneratorUtils::BuildGenerationSettings(false, LocalAsset, *CardsDescriptionPtr,
						OutputSettings.Last().GenerationSettings, OutputSettings.Last().GenerationFlags, OutputSettings.Last().PipelineFlags, AdvancedOptions);
					
					if(!LocalSettings.FilterSettings.IsEmpty())
					{
						TArray<TArray<FName>> FilterCardGroups;
						TArray<FName> FilterGroupNames;
						for(const FGroomFilterSettings& FilterlSettings : LocalSettings.FilterSettings)
						{
							FilterCardGroups.Add(FilterlSettings.CardGroups);
							FilterGroupNames.Add(FilterlSettings.FilterName);
						}
						OutputSettings.Last().GenerationSettings->BuildFilterGroupSettings(FilterCardGroups, FilterGroupNames);
	
						for(const FGroomFilterSettings& FilterlSettings : LocalSettings.FilterSettings)
						{
							for(TObjectPtr<UHairCardGeneratorGroupSettings> FilterGroups : OutputSettings.Last().GenerationSettings->GetFilterGroupSettings())
							{
								if (FilterGroups.Get())
								{
									if ((FilterlSettings.FilterName != NAME_None && FilterGroups->GetFilterName() == FilterlSettings.FilterName) || FilterlSettings.FilterName == NAME_None)
									{
										FilterGroups->TargetNumberOfCards = FilterlSettings.NumClumps * 3;
										FilterGroups->MaxNumberOfFlyaways = FilterlSettings.NumFlyaways;
										FilterGroups->TargetTriangleCount = FilterlSettings.NumTriangles;
										FilterGroups->NumberOfTexturesInAtlas = FilterlSettings.NumTextures;
										
										// Advanced properties
										FilterGroups->UseMultiCardClumps = FilterlSettings.AdvancedOptions.bUseMultiCardClumps;
										FilterGroups->UseAdaptiveSubdivision = FilterlSettings.AdvancedOptions.bUseAdaptiveSubdivision;
										FilterGroups->MaxVerticalSegmentsPerCard = FilterlSettings.AdvancedOptions.MaxVerticalSegmentsPerCard;
										FilterGroups->StrandWidthScalingFactor = FilterlSettings.AdvancedOptions.StrandWidthScalingFactor;
										FilterGroups->UseOptimizedCompressionFactor = FilterlSettings.AdvancedOptions.UseOptimizedCompressionFactor;
									}
								}
							}
						}
						FHairCardGeneratorUtils::LoadGroomStrands(LocalAsset, [&GeometryCollection,LocalAsset](TArray<TArray<FVector>>& CurvesPositions)
						{
							UE::CardGen::Private::BuildStrandsPositions(GeometryCollection, CurvesPositions, LocalAsset->GetHairDescription());
						});
						OutputSettings.Last().GenerationSettings->UpdateStrandFilterAssignment();
					}
				}
			}
		}
		SetValue(Context, MoveTemp(OutputSettings), &CardsSettings);
	}
}


#if WITH_EDITOR
void FBuildCardsSettingsNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		UGroomAsset* LocalAsset = const_cast<UGroomAsset*>(GroomAsset.Get());
		if (!LocalAsset)
		{
			if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
			{
				LocalAsset = Cast<UGroomAsset>(EngineContext->Owner);
			}
		}
		if(LocalAsset)
		{
			FString CardsInfos = FString("Groom Asset Name : ") + LocalAsset->GetName() + FString("\n");
			CardsInfos += UE::CardGen::Private::DisplayCardsGroups(LocalAsset);
			CardsInfos +=  FString("Num Cards Generations : ") + FString::FromInt(LocalAsset->GetHairGroupsCards().Num());
			DataflowRenderingInterface.DrawOverlayText(CardsInfos);
		}
	}
}
#endif

