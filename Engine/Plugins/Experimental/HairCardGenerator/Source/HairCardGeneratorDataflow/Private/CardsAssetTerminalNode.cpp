// Copyright Epic Games, Inc. All Rights Reserved.

#include "CardsAssetTerminalNode.h"

#include "BuildCardsSettingsNode.h"
#include "GroomEdit.h"
#include "HairCardGenControllerBase.h"
#include "HairCardGeneratorEditorModule.h"
#include "HairCardGeneratorEditorSettings.h"
#include "Dataflow/DataflowObjectInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CardsAssetTerminalNode)


FCardsAssetTerminalNode::FCardsAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&CardsSettings);
	RegisterOutputConnection(&Collection, &Collection);
}

void FCardsAssetTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	if (UGroomAsset* GroomAsset = Cast<UGroomAsset>(Asset.Get()))
	{
		TArray<FGroomCardsSettings> LocalSettings = GetValue<TArray<FGroomCardsSettings>>(Context, &CardsSettings);

		if(GroomAsset && !GroomAsset->GetHairGroupsCards().IsEmpty())
		{
			int32 SettingsIndex = 0;
			for(int32 DescriptionIndex = 0, NumDescriptions = GroomAsset->GetHairGroupsCards().Num(); DescriptionIndex < NumDescriptions; ++DescriptionIndex)
			{
				// Use a copy so we can only apply changes on success
				FHairGroupsCardsSourceDescription HairCardsDescription = GroomAsset->GetHairGroupsCards()[DescriptionIndex];
				if(HairCardsDescription.LODIndex != INDEX_NONE && LocalSettings.IsValidIndex(SettingsIndex))
				{ 
					const bool bSuccess = FHairCardGeneratorUtils::BuildCardsAssets(GroomAsset, HairCardsDescription,
						LocalSettings[SettingsIndex].GenerationSettings, LocalSettings[SettingsIndex].GenerationFlags);

					if (bSuccess)
					{
						GroomAsset->Modify();
						GroomAsset->GetHairGroupsCards()[DescriptionIndex] = HairCardsDescription;
					}
					++SettingsIndex;
				}
			}
		}
	}
}

void FCardsAssetTerminalNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
	SetValue(Context, InCollection, &Collection);
}



