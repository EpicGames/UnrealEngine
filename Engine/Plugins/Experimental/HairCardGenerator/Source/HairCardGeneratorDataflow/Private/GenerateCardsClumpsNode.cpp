// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenerateCardsClumpsNode.h"
#include "HairCardGeneratorEditorModule.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowDebugDrawObject.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "Generators/MinimalBoxMeshGenerator.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateCardsClumpsNode)

// Clumps Attributes
const FName FGenerateCardsClumpsNode::CurveClumpIndicesAttribute("ClumpIndices_LOD");
const FName FGenerateCardsClumpsNode::ObjectNumClumpsAttribute("NumClumps_LOD");
const FName FGenerateCardsClumpsNode::CurveFilterIndicesAttribute("FilterIndices_LOD");

FGenerateCardsClumpsNode::FGenerateCardsClumpsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&CardsSettings);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&CardsSettings, &CardsSettings);
}

void FGenerateCardsClumpsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection) || Out->IsA<TArray<FGroomCardsSettings>>(&CardsSettings))
	{
		TArray<FGroomCardsSettings> OutputSettings = GetValue<TArray<FGroomCardsSettings>>(Context, &CardsSettings);
		for(FGroomCardsSettings& LODSettings : OutputSettings)
		{
			// Override the generation settings if matching the LOD index and card group
			if(TObjectPtr<UHairCardGeneratorPluginSettings>
				GenerationSettings = LODSettings.GenerationSettings)
			{
				for(const FCardsClumpsSettings& OverideSettings : ClumpsSettings)
				{
					for(TObjectPtr<UHairCardGeneratorGroupSettings> FilterSettings : GenerationSettings->GetFilterGroupSettings())
					{
						if(FilterSettings.Get())
						{
							if((OverideSettings.FilterName != NAME_None && FilterSettings->GetFilterName() == OverideSettings.FilterName) || OverideSettings.FilterName == NAME_None)
							{
								FilterSettings->TargetNumberOfCards = OverideSettings.NumCards * 3;
								FilterSettings->MaxNumberOfFlyaways = OverideSettings.NumFlyaways;
							}
						}
					}
				}
			}
		}
		if(Out->IsA<TArray<FGroomCardsSettings>>(&CardsSettings))
		{
			SetValue(Context, MoveTemp(OutputSettings), &CardsSettings);
		}
		else if(Out->IsA<FManagedArrayCollection>(&Collection))
		{
			FManagedArrayCollection GeometryCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
			
			for(FGroomCardsSettings& LODSettings : OutputSettings)
			{
				if(TObjectPtr<UHairCardGeneratorPluginSettings> GenerationSettings = LODSettings.GenerationSettings)
				{
					if(FHairCardGeneratorUtils::LoadGenerationSettings(GenerationSettings))
					{
						TArray<int32> StrandsClumps;
						TArray<int32> StrandsFilter;
						int32 NumClumps = 0;

						StrandsClumps.Init(INDEX_NONE, GeometryCollection.NumElements(GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup));
						StrandsFilter.Init(INDEX_NONE, GeometryCollection.NumElements(GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup));
						
						bool bHasClumps = FHairCardGeneratorUtils::RunCardsGeneration(GenerationSettings, LODSettings.PipelineFlags,
				[&StrandsClumps, &NumClumps, &StrandsFilter](const TObjectPtr<const UHairCardGeneratorPluginSettings>& GenerationSettings, const int32 FilterIndex, const uint8 GenFlags)
						{
							FHairCardClumpData ClumpData;
							if(FHairCardGeneratorUtils::GenerateCardsClumps(GenerationSettings, FilterIndex, GenFlags, ClumpData))
							{
								for (int32 CurveIndex = 0; CurveIndex < ClumpData.StrandLabels.Num(); ++CurveIndex)
								{
									if (StrandsClumps.IsValidIndex(CurveIndex) && ClumpData.StrandLabels[CurveIndex] != INDEX_NONE)
									{
										StrandsClumps[CurveIndex] = ClumpData.StrandLabels[CurveIndex] + NumClumps;
										StrandsFilter[CurveIndex] = FilterIndex;
									}
								}
								NumClumps += ClumpData.MaxMainClump + 1;
								return true;
							}
							return false;
						}, false);
			
 						if(bHasClumps)
						{
							FString ClumpIndicesLOD = CurveClumpIndicesAttribute.ToString();
							ClumpIndicesLOD.AppendInt(GenerationSettings->GetLODIndex());
			
							FString NumClumpsLOD = ObjectNumClumpsAttribute.ToString();
							NumClumpsLOD.AppendInt(GenerationSettings->GetLODIndex());

 							FString FilterIndicesLOD = CurveFilterIndicesAttribute.ToString();
 							FilterIndicesLOD.AppendInt(GenerationSettings->GetLODIndex());
					
							TManagedArray<int32>& CurveClumpIndices = GeometryCollection.AddAttribute<int32>(FName(ClumpIndicesLOD), GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup);
							TManagedArray<int32>& ObjectNumClumps = GeometryCollection.AddAttribute<int32>(FName(NumClumpsLOD), FGeometryCollection::GeometryGroup);
 							TManagedArray<int32>& CurveFilterIndices = GeometryCollection.AddAttribute<int32>(FName(FilterIndicesLOD), GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup);
 							
							for(int32 CurveIndex = 0, NumCurves = CurveClumpIndices.Num(); CurveIndex < NumCurves; ++CurveIndex)
							{
								CurveClumpIndices[CurveIndex] = StrandsClumps[CurveIndex];
								CurveFilterIndices[CurveIndex] = StrandsFilter[CurveIndex];
							}
							for(int32 ObjectIndex = 0, NumObjects = ObjectNumClumps.Num(); ObjectIndex < NumObjects; ++ObjectIndex)
							{
								ObjectNumClumps[ObjectIndex] = NumClumps;
							}
						}
					}
				}
			}
			SetValue(Context, MoveTemp(GeometryCollection), &Collection);
		}
	}
}

