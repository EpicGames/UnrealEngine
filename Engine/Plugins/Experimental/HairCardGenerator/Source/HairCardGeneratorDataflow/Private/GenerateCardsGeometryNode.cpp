// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenerateCardsGeometryNode.h"
#include "GenerateCardsClumpsNode.h"
#include "HairCardGeneratorEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateCardsGeometryNode)

// Geometry Attributes
const FName FGenerateCardsGeometryNode::VertexClumpPositionsAttribute("VertexClumpPositions");
const FName FGenerateCardsGeometryNode::FaceVertexIndicesAttribute("FaceVertexIndices");
const FName FGenerateCardsGeometryNode::VertexCardIndicesAttribute("VertexCardIndices");

const FName FGenerateCardsGeometryNode::CardsVerticesGroup("CardsVertices_LOD");
const FName FGenerateCardsGeometryNode::CardsFacesGroup("CardsFaces_LOD");

FGenerateCardsGeometryNode::FGenerateCardsGeometryNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&CardsSettings);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&CardsSettings, &CardsSettings);
}

void FGenerateCardsGeometryNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection) || Out->IsA<TArray<FGroomCardsSettings>>(&CardsSettings))
	{
		TArray<FGroomCardsSettings> OutputSettings = GetValue<TArray<FGroomCardsSettings>>(Context, &CardsSettings);
		for(FGroomCardsSettings& LODSettings : OutputSettings)
		{
			// Override the generation settings if matching the LOD index and card group
			if(TObjectPtr<UHairCardGeneratorPluginSettings> GenerationSettings = LODSettings.GenerationSettings)
			{
				for(const FCardsGeometrySettings& OverideSettings : GeometrySettings)
				{
					for(TObjectPtr<UHairCardGeneratorGroupSettings> FilterSettings : GenerationSettings->GetFilterGroupSettings())
					{
						if(FilterSettings.Get())
                        {
							if((OverideSettings.FilterName != NAME_None && FilterSettings->GetFilterName() == OverideSettings.FilterName) || OverideSettings.FilterName == NAME_None)
							{
								FilterSettings->TargetTriangleCount = OverideSettings.NumTriangles;
							}
						}
					}
				}
			}
		}
		if(Out->IsA<TArray<FGroomCardsSettings>>(&CardsSettings))
		{
			SetValue(Context, MoveTemp(OutputSettings), &CardsSettings);
		}
		else if(Out->IsA<FManagedArrayCollection>(&Collection))
		{
			FManagedArrayCollection GeometryCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
			
			for(FGroomCardsSettings& LODSettings : OutputSettings)
			{
				if(TObjectPtr<UHairCardGeneratorPluginSettings> GenerationSettings = LODSettings.GenerationSettings)
				{
					if(FHairCardGeneratorUtils::LoadGenerationSettings(GenerationSettings))
					{
						TArray<FIntVector3> ClumpsFaces;
						TArray<FVector3f> ClumpsVertices;
						TArray<int32> CardIndices;
						int32 GlobalVertexOffset = 0;
						int32 GlobalCardIndex = 0;

						bool bHasGeometry = FHairCardGeneratorUtils::RunCardsGeneration(GenerationSettings, LODSettings.PipelineFlags,
							[&ClumpsFaces, &ClumpsVertices, &CardIndices, &GlobalVertexOffset, &GlobalCardIndex]
							(const TObjectPtr<const UHairCardGeneratorPluginSettings>& GenerationSettings, const int32 FilterIndex, const uint8 GenFlags)
						{
							FHairCardGeomData GeomData;
							FHairCardMeshData MeshData;
							int32 CardCount = 0;
							if(!FHairCardGeneratorUtils::GenerateCardsGeometry(GenerationSettings, FilterIndex, GenFlags, GeomData, MeshData, CardCount))
								return false;

							for (int32 c = 0; c < CardCount; ++c)
							{
								const int32 v0 = MeshData.CardVertOffsets[c];
								const int32 vc = MeshData.CardVertCounts[c];
								const int32 f0 = MeshData.CardFaceOffsets[c];
								const int32 fc = MeshData.CardFaceCounts[c];

								for (int32 j = 0; j < vc; ++j)
								{
									const int32 Base = (v0 + j) * 3;
									ClumpsVertices.Add(FVector3f(MeshData.Verts[Base], MeshData.Verts[Base+1], MeshData.Verts[Base+2]));
									CardIndices.Add(GlobalCardIndex);
								}
								// Face indices are card-local (0..vc-1); add GlobalVertexOffset for collection indices
								for (int32 j = 0; j < fc; ++j)
								{
									const int32 Base = (f0 + j) * 3;
									ClumpsFaces.Add(FIntVector3(
										MeshData.Faces[Base]     + GlobalVertexOffset,
										MeshData.Faces[Base + 1] + GlobalVertexOffset,
										MeshData.Faces[Base + 2] + GlobalVertexOffset));
								}
								GlobalVertexOffset += vc;
								++GlobalCardIndex;
							}
							return true;
						}, false);

						if(bHasGeometry)
						{
							
							FString CardsVerticesLODGroup = CardsVerticesGroup.ToString();
							CardsVerticesLODGroup.AppendInt(GenerationSettings->GetLODIndex());
							
							FString CardsFacesLODGroup = CardsFacesGroup.ToString();
							CardsFacesLODGroup.AppendInt(GenerationSettings->GetLODIndex());
							
							TManagedArray<FVector3f>& VertexClumpPositions = GeometryCollection.AddAttribute<FVector3f>(VertexClumpPositionsAttribute, FName(CardsVerticesLODGroup));
							TManagedArray<FIntVector3>& FaceVertexIndices = GeometryCollection.AddAttribute<FIntVector3>(FaceVertexIndicesAttribute, FName(CardsFacesLODGroup));
							TManagedArray<int32>& VertexCardIndices = GeometryCollection.AddAttribute<int32>(VertexCardIndicesAttribute, FName(CardsVerticesLODGroup));
							
							GeometryCollection.EmptyGroup(FName(CardsVerticesLODGroup));
							GeometryCollection.AddElements(ClumpsVertices.Num(), FName(CardsVerticesLODGroup));
							
							GeometryCollection.EmptyGroup(FName(CardsFacesLODGroup));
                            GeometryCollection.AddElements(ClumpsFaces.Num(), FName(CardsFacesLODGroup));
							
							for(int32 VertexIndex = 0, NumVertices = ClumpsVertices.Num(); VertexIndex < NumVertices; ++VertexIndex)
							{
								VertexClumpPositions[VertexIndex] = ClumpsVertices[VertexIndex];
								VertexCardIndices[VertexIndex] = CardIndices[VertexIndex];
							}
							for(int32 FaceIndex = 0, NumFaces = ClumpsFaces.Num(); FaceIndex < NumFaces; ++FaceIndex)
							{
								FaceVertexIndices[FaceIndex] = ClumpsFaces[FaceIndex];
							}
						}
					}
				}
			}
			SetValue(Context, MoveTemp(GeometryCollection), &Collection);
		}
	}
}

