// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenerateCardsTexturesNode.h"

#include "GenerateCardsGeometryNode.h"
#include "HairCardGeneratorEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateCardsTexturesNode)

// Texture Attributes
const FName FGenerateCardsTexturesNode::ObjectTextureIndicesAttribute("ObjectTextureIndices");
const FName FGenerateCardsTexturesNode::CardsObjectsGroup("CardsObjects_LOD");
const FName FGenerateCardsTexturesNode::VertexTextureUVsAttribute("VertexTextureUVs");

FGenerateCardsTexturesNode::FGenerateCardsTexturesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&CardsSettings);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&CardsSettings, &CardsSettings);
}

void FGenerateCardsTexturesNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection) || Out->IsA<TArray<FGroomCardsSettings>>(&CardsSettings))
	{
		TArray<FGroomCardsSettings> OutputSettings = GetValue<TArray<FGroomCardsSettings>>(Context, &CardsSettings);
		for(FGroomCardsSettings& LODSettings : OutputSettings)
		{
			// Override the generation settings if matching the LOD index and card group
			if(TObjectPtr<UHairCardGeneratorPluginSettings> GenerationSettings = LODSettings.GenerationSettings)
			{
				for(const FCardsTextureSettings& OverideSettings : TextureSettings)
				{
					for(TObjectPtr<UHairCardGeneratorGroupSettings> FilterSettings : GenerationSettings->GetFilterGroupSettings())
					{
						if (FilterSettings.Get())
						{
							if ((OverideSettings.FilterName != NAME_None && FilterSettings->GetFilterName() == OverideSettings.FilterName) || OverideSettings.FilterName == NAME_None)
							{
								FilterSettings->NumberOfTexturesInAtlas = OverideSettings.NumTextures;
							}
						}
					}
				}
			}
		}
		if(Out->IsA<TArray<FGroomCardsSettings>>(&CardsSettings))
		{
			SetValue(Context, MoveTemp(OutputSettings), &CardsSettings);
		}
		else if (Out->IsA<FManagedArrayCollection>(&Collection))
		{
			FManagedArrayCollection GroomCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

			for(FGroomCardsSettings& LODSettings : OutputSettings)
			{
				if(TObjectPtr<UHairCardGeneratorPluginSettings> GenerationSettings = LODSettings.GenerationSettings)
				{
					if(FHairCardGeneratorUtils::LoadGenerationSettings(GenerationSettings))
					{
						// Accumulate per-group cluster data and card offsets across RunCardsGeneration groups
						TArray<FHairCardTextureClusterData> AllClusterData;
						TArray<int32> CardOffsets;
						CardOffsets.Add(0);

						bool bHasTextures = FHairCardGeneratorUtils::RunCardsGeneration(GenerationSettings, LODSettings.PipelineFlags,
							[&AllClusterData, &CardOffsets](const TObjectPtr<const UHairCardGeneratorPluginSettings>& GenerationSettings, const int32 FilterIndex, const uint8 GenFlags)
						{
							FHairCardTextureClusterData ClusterData;
							if(!FHairCardGeneratorUtils::GenerateCardsTexturesClusters(GenerationSettings, FilterIndex, GenFlags, ClusterData))
								return false;
							CardOffsets.Add(CardOffsets.Last() + ClusterData.ClustersLabel.Num());
							AllClusterData.Add(MoveTemp(ClusterData));
							return true;
						}, false);

						if(bHasTextures)
						{
							FHairCardAtlasLayoutData LayoutData;
							FHairCardAtlasUVData AtlasUVData;
							if(FHairCardGeneratorUtils::GenerateTexturesLayoutAndAtlases(GenerationSettings, LODSettings.PipelineFlags, LayoutData, AtlasUVData))
							{
								// Build ObjectTextureIndices: per card, global center card ID (or INDEX_NONE for flyaways)
								// Center cards satisfy ObjectTextureIndices[card] == card, enabling cluster vis in the renderer
								TArray<int32> CardsTextures;
								for(int32 Gid = 0; Gid < AllClusterData.Num(); ++Gid)
								{
									const FHairCardTextureClusterData& CD = AllClusterData[Gid];
									const int32 CardOffset = CardOffsets[Gid];
									const int32 NumClusters = CD.ClustersCenterId.Num();
									for(const int32 ClusterIdx : CD.ClustersLabel)
									{
										CardsTextures.Add((ClusterIdx >= 0 && ClusterIdx < NumClusters)
											? CD.ClustersCenterId[ClusterIdx] + CardOffset
											: INDEX_NONE);
									}
								}

								// Build VertexUVs directly from the flat AtlasUVData — no sentinel parsing needed
								TArray<FVector2f> VertexUVs;
								VertexUVs.Reserve(AtlasUVData.VertexUvs.Num() / 2);
								for(int32 i = 0; i < AtlasUVData.VertexUvs.Num(); i += 2)
								{
									VertexUVs.Add(FVector2f(AtlasUVData.VertexUvs[i], AtlasUVData.VertexUvs[i + 1]));
								}

								FString CardsObjectsLODGroup = CardsObjectsGroup.ToString();
								CardsObjectsLODGroup.AppendInt(GenerationSettings->GetLODIndex());

								FString CardsVerticesLODGroup = FGenerateCardsGeometryNode::CardsVerticesGroup.ToString();
								CardsVerticesLODGroup.AppendInt(GenerationSettings->GetLODIndex());

								TManagedArray<int32>& ObjectTextureIndices = GroomCollection.AddAttribute<int32>(ObjectTextureIndicesAttribute, FName(CardsObjectsLODGroup));
								TManagedArray<FVector2f>& VertexTextureUvs = GroomCollection.AddAttribute<FVector2f>(VertexTextureUVsAttribute, FName(CardsVerticesLODGroup));

								GroomCollection.EmptyGroup(FName(CardsObjectsLODGroup));
								GroomCollection.AddElements(CardsTextures.Num(), FName(CardsObjectsLODGroup));

								for(int32 CardIndex = 0, NumCards = CardsTextures.Num(); CardIndex < NumCards; ++CardIndex)
								{
									ObjectTextureIndices[CardIndex] = CardsTextures[CardIndex];
								}

								if(VertexUVs.Num() == VertexTextureUvs.Num())
								{
									for(int32 VertexIndex = 0, NumVertices = VertexUVs.Num(); VertexIndex < NumVertices; ++VertexIndex)
									{
										VertexTextureUvs[VertexIndex] = VertexUVs[VertexIndex];
									}
								}
							}
						}
					}
				}
			}
			SetValue(Context, MoveTemp(GroomCollection), &Collection);
		}
	}
}

