// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardDataflowRendering.h"
#include "Components/DynamicMeshComponent.h"
#include "GenerateCardsGeometryNode.h"
#include "GenerateCardsClumpsNode.h"
#include "GenerateCardsTexturesNode.h"
#include "PlanarCut.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "Materials/Material.h"

#define LOCTEXT_NAMESPACE "CardsRendering"

namespace UE::CardGen::Private
{
	int32 GGroomDataflowCardsLod = 0;
	static FAutoConsoleVariableRef CVarGroomDataflowCardsLod(
		 TEXT("p.Groom.Dataflow.CardsLod"),
		 GGroomDataflowCardsLod,
		 TEXT("Cards Lod we want to display")
	 );

	float GGroomDataflowCardsAlpha = 0.1f;
	static FAutoConsoleVariableRef CVarGroomDataflowCardsAlpha(
		 TEXT("p.Groom.Dataflow.CardsAlpha"),
		 GGroomDataflowCardsAlpha,
		 TEXT("Cards Alpha for the rendering")
	 );

	static void BuildGeometryCollection(const TArray<FVector3f>& InVertexPosisions, const TArray<FVector3f>& InVertexNormals,
		const TArray<FIntVector3>& InFaceIndices, const TArray<FLinearColor>& InVertexColors, const FName& GeometryName, Geometry::FDynamicMesh3& DynamicMesh)
	{
		FGeometryCollection GeometryCollection;
		
		// Transform group
		GeometryCollection.EmptyGroup(FGeometryCollection::TransformGroup);
		GeometryCollection.AddElements(1, FGeometryCollection::TransformGroup);

		// Geometry group
		GeometryCollection.EmptyGroup(FGeometryCollection::GeometryGroup);
		GeometryCollection.AddElements(1, FGeometryCollection::GeometryGroup);

		// Vertices group
		GeometryCollection.EmptyGroup(FGeometryCollection::VerticesGroup);
		GeometryCollection.AddElements(InVertexPosisions.Num(), FGeometryCollection::VerticesGroup);

		// Faces group
		GeometryCollection.EmptyGroup(FGeometryCollection::FacesGroup);
		GeometryCollection.AddElements(InFaceIndices.Num(), FGeometryCollection::FacesGroup);

		// Material group
		GeometryCollection.EmptyGroup(FGeometryCollection::MaterialGroup);

		// Vertices Attributes
		TManagedArray<FVector3f>& VertexPositions = GeometryCollection.ModifyAttribute<FVector3f>(FGeometryCollection::VertexPositionAttribute, FGeometryCollection::VerticesGroup);
		TManagedArray<FVector3f>& VertexNormals = GeometryCollection.ModifyAttribute<FVector3f>(FGeometryCollection::VertexNormalAttribute, FGeometryCollection::VerticesGroup);
		TManagedArray<FVector3f>& VertexTangentU = GeometryCollection.ModifyAttribute<FVector3f>(FGeometryCollection::VertexTangentUAttribute, FGeometryCollection::VerticesGroup);
		TManagedArray<FVector3f>& VertexTangentV = GeometryCollection.ModifyAttribute<FVector3f>(FGeometryCollection::VertexTangentVAttribute, FGeometryCollection::VerticesGroup);
		TManagedArray<FLinearColor>& VertexColors = GeometryCollection.ModifyAttribute<FLinearColor>(FGeometryCollection::ColorAttribute, FGeometryCollection::VerticesGroup);
		TManagedArray<int32>& VertexTransforms = GeometryCollection.ModifyAttribute<int32>(FGeometryCollection::VertexBoneMapAttribute, FGeometryCollection::VerticesGroup);
		
		// Faces Attributes
		TManagedArray<FIntVector>& FaceIndices = GeometryCollection.ModifyAttribute<FIntVector>(FGeometryCollection::FaceIndicesAttribute, FGeometryCollection::FacesGroup);
		TManagedArray<bool>& FaceVisibles = GeometryCollection.ModifyAttribute<bool>(FGeometryCollection::FaceVisibleAttribute, FGeometryCollection::FacesGroup);

		// Geometry Group
		TManagedArray<int32>& TransformIndex = GeometryCollection.ModifyAttribute<int32>(FGeometryCollection::TransformIndexAttribute, FGeometryCollection::GeometryGroup);
		TManagedArray<FBox>& BoundingBox =  GeometryCollection.ModifyAttribute<FBox>(FGeometryCollection::BoundingBoxAttribute, FGeometryCollection::GeometryGroup);
		TManagedArray<int32>& VertexStart = GeometryCollection.ModifyAttribute<int32>(FGeometryCollection::VertexStartAttribute, FGeometryCollection::GeometryGroup);
		TManagedArray<int32>& VertexCount = GeometryCollection.ModifyAttribute<int32>(FGeometryCollection::VertexCountAttribute, FGeometryCollection::GeometryGroup);
		TManagedArray<int32>& FaceStart = GeometryCollection.ModifyAttribute<int32>(FGeometryCollection::FaceStartAttribute, FGeometryCollection::GeometryGroup);
		TManagedArray<int32>& FaceCount = GeometryCollection.ModifyAttribute<int32>(FGeometryCollection::FaceCountAttribute, FGeometryCollection::GeometryGroup);

		// Transform group
		TManagedArray<int32>& BoneGeometry = GeometryCollection.ModifyAttribute<int32>(FTransformCollection::GeometryIndexAttribute, FTransformCollection::TransformGroup);
		TManagedArray<FTransform3f>& BoneTransforms = GeometryCollection.ModifyAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
		TManagedArray<FString>& BoneNames = GeometryCollection.ModifyAttribute<FString>(FTransformCollection::BoneNameAttribute, FTransformCollection::TransformGroup);
		TManagedArray<int32>& BoneParent = GeometryCollection.ModifyAttribute<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);

		VertexTangentU.Fill(FVector3f::Zero());
		VertexTangentV.Fill(FVector3f::Zero());
		VertexTransforms.Fill(0);
		FaceVisibles.Fill(true);
		
		VertexNormals = InVertexNormals;
		VertexPositions = InVertexPosisions;
		VertexColors = InVertexColors;
		FaceIndices = InFaceIndices;
		
		BoundingBox[0] = FBox(ForceInitToZero);
		TransformIndex[0] = 0;
		BoneGeometry[0] = 0;
		BoneTransforms[0] = FTransform3f::Identity;
		BoneParent[0] = INDEX_NONE;
		BoneNames[0] = GeometryName.ToString();
		VertexStart[0] = 0;
		FaceStart[0] = 0;
		VertexCount[0] = InVertexPosisions.Num();
		FaceCount[0] = InFaceIndices.Num();

		for(int32 VertexIndex = 0; VertexIndex < InVertexPosisions.Num(); ++VertexIndex)
		{
			BoundingBox[0] += FVector(VertexPositions[VertexIndex]);
		}
		TArray<int32> TransformIndices;
		TransformIndices.Add(0);

		TArrayView<const int32> TransformIndicesView = MakeArrayView(TransformIndices);
		TArrayView<const FTransform3f> BoneTransformsView = MakeArrayView(BoneTransforms.GetConstArray());

		FTransform UnusedTransform;
		::ConvertGeometryCollectionToDynamicMesh(DynamicMesh, UnusedTransform, false, GeometryCollection, true, BoneTransformsView, true, TransformIndicesView);
	}
	
	static void AddMeshComponents(TArray<Geometry::FDynamicMesh3>& DynamicMeshes, Dataflow::FRenderableComponents& OutComponents, const TArray<TArray<int32>>& FilterTransforms)
	{
		if(!FilterTransforms.IsEmpty())
		{
			for(const TArray<int32>& TransformsIndices : FilterTransforms)
			{
				if (!TransformsIndices.IsEmpty())
				{
					int32 TransformIndex = TransformsIndices[0];
					if (UDynamicMeshComponent* FilterComponent = OutComponents.AddNewComponent<UDynamicMeshComponent>())
					{
						if (DynamicMeshes[TransformIndex].VertexCount() > 0)
						{
							FilterComponent->SetMesh(MoveTemp(DynamicMeshes[TransformIndex]));
							FilterComponent->SetOverrideRenderMaterial(Dataflow::Rendering::GetVertexMaterial());
							FilterComponent->SetCastShadow(false);
						}

						for (int32 LocalIndex = 1; LocalIndex < TransformsIndices.Num(); ++LocalIndex)
						{
							TransformIndex = TransformsIndices[LocalIndex];
							if (DynamicMeshes[TransformIndex].VertexCount() > 0)
							{
								if (UDynamicMeshComponent* Component = OutComponents.AddNewComponent<UDynamicMeshComponent>(NAME_None, FilterComponent))
								{
									Component->SetupAttachment(FilterComponent);
									Component->SetMesh(MoveTemp(DynamicMeshes[TransformIndex]));
									Component->SetOverrideRenderMaterial(Dataflow::Rendering::GetVertexMaterial());
									Component->SetCastShadow(false);
								}
							}
						}
					}
				}
			}
		}
		else
		{
			for (int32 TransformIndex = 0; TransformIndex < DynamicMeshes.Num(); ++TransformIndex)
			{
				if (DynamicMeshes[TransformIndex].VertexCount() > 0)
				{
					if (UDynamicMeshComponent* Component = OutComponents.AddNewComponent<UDynamicMeshComponent>())
					{
						Component->SetMesh(MoveTemp(DynamicMeshes[TransformIndex]));
						Component->SetOverrideRenderMaterial(Dataflow::Rendering::GetVertexMaterial());
						Component->SetCastShadow(false);
					}
				}
			}
		}
	}

	static void CountClumpsAndFilters(const TArray<int32>& CurveClumpIndices, const TArray<int32>& CurveFilterIndices,
		TArray<TArray<int32>>& FilterClumpIndices, TArray<TArray<int32>>& ClumpCurveIndices, int32& NumClumps, int32& NumFilters)
	{
		for(int32 CurveIndex = 0, NumCurves = CurveClumpIndices.Num(); CurveIndex < NumCurves; ++CurveIndex)
		{
			NumClumps = FMath::Max(NumClumps, CurveClumpIndices[CurveIndex]);
			NumFilters = FMath::Max(NumFilters, CurveFilterIndices[CurveIndex]);
		}
		++NumClumps;
		++NumFilters;
		
		ClumpCurveIndices.SetNum(NumClumps);
		FilterClumpIndices.SetNum(NumFilters);

		// Fill the clump curve indices and filter clump indices
		for(int32 CurveIndex = 0, NumCurves = CurveClumpIndices.Num(); CurveIndex < NumCurves; ++CurveIndex)
		{
			const int32 ClumpIndex = CurveClumpIndices[CurveIndex];
			if ((ClumpIndex >= 0) && ClumpCurveIndices.IsValidIndex(ClumpIndex))
			{
				ClumpCurveIndices[ClumpIndex].Add(CurveIndex);
			}
		}
		for(int32 ClumpIndex = 0; ClumpIndex < NumClumps; ++ClumpIndex)
		{
			if(!ClumpCurveIndices[ClumpIndex].IsEmpty())
			{
				const int32 CurveIndex = ClumpCurveIndices[ClumpIndex].Last();
				if(CurveFilterIndices.IsValidIndex(CurveIndex))
				{
					const int32 FilterIndex = CurveFilterIndices[CurveIndex];
					if(FilterClumpIndices.IsValidIndex(FilterIndex))
					{
						FilterClumpIndices[FilterIndex].Add(ClumpIndex);
					}
				}
			}
		}
	}

	static void BuildCardsGeometry(const FManagedArrayCollection& Collection, TArray<TArray<FVector3f>>& VertexPositions, TArray<TArray<FVector3f>>& VertexNormals, TArray<TArray<FIntVector3>>& FaceVertices)
	{
		FString CardsVerticesLODGroup = FGenerateCardsGeometryNode::CardsVerticesGroup.ToString();
		CardsVerticesLODGroup.AppendInt(GGroomDataflowCardsLod);
					
		FString CardsFacesLODGroup = FGenerateCardsGeometryNode::CardsFacesGroup.ToString();
		CardsFacesLODGroup.AppendInt(GGroomDataflowCardsLod);

		if (Collection.HasAttribute(FGenerateCardsGeometryNode::VertexClumpPositionsAttribute, FName(CardsVerticesLODGroup)) &&
			Collection.HasAttribute(FGenerateCardsGeometryNode::FaceVertexIndicesAttribute, FName(CardsFacesLODGroup)) && 
			Collection.HasAttribute(FGenerateCardsGeometryNode::VertexCardIndicesAttribute, FName(CardsVerticesLODGroup)))
		{
			const TManagedArray<FVector3f>& VertexGlobalPositions = Collection.GetAttribute<FVector3f>(
				FGenerateCardsGeometryNode::VertexClumpPositionsAttribute, FName(CardsVerticesLODGroup));
				
			const TManagedArray<FIntVector3>& FaceGlobalVertices = Collection.GetAttribute<FIntVector3>(
				FGenerateCardsGeometryNode::FaceVertexIndicesAttribute, FName(CardsFacesLODGroup));

			const TManagedArray<int32>& VertexCardIndices = Collection.GetAttribute<int32>(
				FGenerateCardsGeometryNode::VertexCardIndicesAttribute, FName(CardsVerticesLODGroup));

			int32 NumCards = 0;
			for(const int32& CardIndex : VertexCardIndices.GetConstArray())
			{
				NumCards = FMath::Max(NumCards, CardIndex);
			}
			++NumCards;

			TArray<TArray<int32>> CardVertexIndices;
			TArray<TArray<int32>> CardFaceIndices;
			TArray<int32> VertexLocalIndices;

			CardVertexIndices.SetNum(NumCards);
			CardFaceIndices.SetNum(NumCards);
			VertexLocalIndices.SetNum(VertexCardIndices.Num());
			
			for (int32 VertexIndex = 0, NumVertices = VertexCardIndices.Num(); VertexIndex < NumVertices; ++VertexIndex)
			{
				VertexLocalIndices[VertexIndex] = CardVertexIndices[VertexCardIndices[VertexIndex]].Num();
				CardVertexIndices[VertexCardIndices[VertexIndex]].Add(VertexIndex);
			}

			for (int32 FaceIndex = 0, NumFaces = FaceGlobalVertices.Num(); FaceIndex < NumFaces; ++FaceIndex)
			{
				CardFaceIndices[VertexCardIndices[FaceGlobalVertices[FaceIndex][0]]].Add(FaceIndex);
			}
			VertexPositions.SetNum(NumCards);
			VertexNormals.SetNum(NumCards);
			FaceVertices.SetNum(NumCards);
			for(int32 CardIndex = 0; CardIndex < NumCards; ++CardIndex)
			{
				TArray<int32> VertexNumFaces;

				VertexPositions[CardIndex].Init(FVector3f::Zero(), CardVertexIndices[CardIndex].Num());
				VertexNormals[CardIndex].Init(FVector3f::Zero(), CardVertexIndices[CardIndex].Num());
				VertexNumFaces.Init(0, CardVertexIndices[CardIndex].Num());
				FaceVertices[CardIndex].Init(FIntVector::ZeroValue, CardFaceIndices[CardIndex].Num());

				for (int32 FaceIndex = 0, NumFaces = CardFaceIndices[CardIndex].Num(); FaceIndex < NumFaces; ++FaceIndex)
				{
					const FIntVector3 GlobalVertices = FaceGlobalVertices[CardFaceIndices[CardIndex][FaceIndex]];
					FaceVertices[CardIndex][FaceIndex] = FIntVector(VertexLocalIndices[GlobalVertices[0]], VertexLocalIndices[GlobalVertices[1]], VertexLocalIndices[GlobalVertices[2]]);

					const FVector3f FaceNormal = (VertexGlobalPositions[GlobalVertices[2]] - VertexGlobalPositions[GlobalVertices[0]]).Cross(
						VertexGlobalPositions[GlobalVertices[1]] - VertexGlobalPositions[GlobalVertices[0]]).GetSafeNormal();

					VertexNormals[CardIndex][FaceVertices[CardIndex][FaceIndex][0]] += FaceNormal;
					VertexNormals[CardIndex][FaceVertices[CardIndex][FaceIndex][1]] += FaceNormal;
					VertexNormals[CardIndex][FaceVertices[CardIndex][FaceIndex][2]] += FaceNormal;

					VertexNumFaces[FaceVertices[CardIndex][FaceIndex][0]]++;
					VertexNumFaces[FaceVertices[CardIndex][FaceIndex][1]]++;
					VertexNumFaces[FaceVertices[CardIndex][FaceIndex][2]]++;
				}

				for (int32 VertexIndex = 0, NumVertices = CardVertexIndices[CardIndex].Num(); VertexIndex < NumVertices; ++VertexIndex)
				{
					if (VertexNumFaces[VertexIndex] > 0)
					{
						VertexNormals[CardIndex][VertexIndex] /= VertexNumFaces[VertexIndex];
					}
					VertexPositions[CardIndex][VertexIndex] = VertexGlobalPositions[CardVertexIndices[CardIndex][VertexIndex]];
				}
			}
		}
	}

	class FGeometryCollectionCardsClumpsRenderableType : public Dataflow::IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FManagedArrayCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(CardsClumps);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(UE::Dataflow::FDataflowConstruction3DViewMode);

		virtual bool CanRender(const Dataflow::FRenderableTypeInstance& Instance) const
		{
			const FManagedArrayCollection& Collection = GetCollection(Instance);
			return Collection.HasGroup(FGeometryCollection::GeometryGroup);
		}

		virtual void GetPrimitiveComponents(const Dataflow::FRenderableTypeInstance& Instance, Dataflow::FRenderableComponents& OutComponents) const override
		{
			TArray<Geometry::FDynamicMesh3> DynamicMeshes;
			
			TArray<TArray<int32>> FilterClumpIndices;
			TArray<TArray<int32>> ClumpCurveIndices;
			
			const FManagedArrayCollection& Collection = GetCollection(Instance);

			FString ClumpIndicesLOD = FGenerateCardsClumpsNode::CurveClumpIndicesAttribute.ToString();
			ClumpIndicesLOD.AppendInt(GGroomDataflowCardsLod);
			
			FString FilterIndicesLOD = FGenerateCardsClumpsNode::CurveFilterIndicesAttribute.ToString();
			FilterIndicesLOD.AppendInt(GGroomDataflowCardsLod);

			if (Collection.HasAttribute(FName(ClumpIndicesLOD), GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup) &&
				Collection.HasAttribute(FName(FilterIndicesLOD), GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup))
			{
				const GeometryCollection::Facades::FCollectionCurveGeometryFacade CurvesFacade(Collection);
				if (CurvesFacade.IsValid())
				{
					const TManagedArray<int32>& CurveClumpIndices = Collection.GetAttribute<int32>(FName(ClumpIndicesLOD), GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup);
					const TManagedArray<int32>& CurveFilterIndices = Collection.GetAttribute<int32>(FName(FilterIndicesLOD), GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup);

					int32 NumClumps = 0, NumFilters = 0;
					CountClumpsAndFilters(CurveClumpIndices.GetConstArray(), CurveFilterIndices.GetConstArray(),
						FilterClumpIndices, ClumpCurveIndices, NumClumps, NumFilters);
					
					if (Instance.HasUptoDateCachedValue())
					{
						DynamicMeshes = Instance.GetCachedValue<TArray<Geometry::FDynamicMesh3>>();
					}
					else
					{
						FString NumClumpsLOD = FGenerateCardsClumpsNode::ObjectNumClumpsAttribute.ToString();
						NumClumpsLOD.AppendInt(GGroomDataflowCardsLod);

						if (Collection.HasAttribute(FName(NumClumpsLOD), FGeometryCollection::GeometryGroup))
						{
							TArray<TArray<TArray<FVector>>> ClumpMeshPoints;
							TArray<TArray<FVector>> ClumpMeshOrigins;
							TArray<TArray<FQuat>> ClumpMeshRotations;
							TArray<TArray<FVector>> ClumpMeshExtentsX;
							TArray<TArray<FVector>> ClumpMeshExtentsY;

							ClumpMeshPoints.SetNum(NumClumps);
							ClumpMeshOrigins.SetNum(NumClumps);
							ClumpMeshRotations.SetNum(NumClumps);
							ClumpMeshExtentsX.SetNum(NumClumps);
							ClumpMeshExtentsY.SetNum(NumClumps);

							static constexpr const int32 NumMeshes = 10;
							static constexpr const int32 NumCorners = 8;
							static constexpr const int32 NumTriangles = 12;
							static constexpr const float MinExtent = 0.5f;
							
							DynamicMeshes.SetNum(NumClumps);
							for(int32 ClumpIndex = 0; ClumpIndex < NumClumps; ++ClumpIndex)
							{
								// Fill the clump points for eaqch meshes
								ClumpMeshPoints[ClumpIndex].SetNum(NumMeshes+1);
								for (const int32& CurveIndex : ClumpCurveIndices[ClumpIndex])
								{
									if(CurvesFacade.GetCurvePointOffsets().IsValidIndex(CurveIndex) && CurvesFacade.GetCurvePointOffsets().IsValidIndex(CurveIndex - 1))
									{
										const float PointStart = (CurveIndex > 0) ? CurvesFacade.GetCurvePointOffsets()[CurveIndex - 1] : 0;
										const float PointEnd = CurvesFacade.GetCurvePointOffsets()[CurveIndex]-1;

										if(CurvesFacade.GetPointRestPositions().IsValidIndex(PointStart) && CurvesFacade.GetPointRestPositions().IsValidIndex(PointEnd))
										{
											ClumpMeshPoints[ClumpIndex][0].Add(FVector(CurvesFacade.GetPointRestPositions()[PointStart]));
											ClumpMeshPoints[ClumpIndex][NumMeshes].Add(FVector(CurvesFacade.GetPointRestPositions()[PointEnd]));

											for(int32 MeshIndex = 1; MeshIndex < NumMeshes; ++MeshIndex)
											{
												const float PointParam = PointStart + (PointEnd - PointStart) * MeshIndex / NumMeshes;
												const int32 ParamIndex = FMath::FloorToInt(PointParam);
												const float ParamAlpha = PointParam - ParamIndex;

												if(CurvesFacade.GetPointRestPositions().IsValidIndex(ParamIndex) && CurvesFacade.GetPointRestPositions().IsValidIndex(ParamIndex+1))
												{
													ClumpMeshPoints[ClumpIndex][MeshIndex].Add(FVector(CurvesFacade.GetPointRestPositions()[ParamIndex] * (1.0 - ParamAlpha) + CurvesFacade.GetPointRestPositions()[ParamIndex+1] * ParamAlpha));
												}
											}
										}
									}
								}

								// Compute the clump mesh origins
								ClumpMeshOrigins[ClumpIndex].Init(FVector::ZeroVector, NumMeshes + 1);
								for (int32 MeshIndex = 0; MeshIndex < NumMeshes+1; ++MeshIndex)
								{
									if(!ClumpMeshPoints[ClumpIndex][MeshIndex].IsEmpty())
									{ 
										for (const FVector& MeshPoint : ClumpMeshPoints[ClumpIndex][MeshIndex])
										{
											ClumpMeshOrigins[ClumpIndex][MeshIndex] += MeshPoint;
										}
										ClumpMeshOrigins[ClumpIndex][MeshIndex] /= ClumpMeshPoints[ClumpIndex][MeshIndex].Num();
									}
								}

								FVector NextTangent = FVector(0,0,1), PrevTangent = FVector(0, 0, 1);
								FQuat MeshOrientation = FQuat::Identity;

								// Compute the clump mesh rotations
								ClumpMeshRotations[ClumpIndex].Init(FQuat::Identity, NumMeshes);
								for (int32 MeshIndex = 0; MeshIndex < NumMeshes; ++MeshIndex)
								{
									PrevTangent = NextTangent;
									NextTangent = (ClumpMeshOrigins[ClumpIndex][MeshIndex+1]-ClumpMeshOrigins[ClumpIndex][MeshIndex]).GetSafeNormal();

									MeshOrientation = (FQuat::FindBetweenNormals(PrevTangent, NextTangent) * MeshOrientation).GetNormalized();
									ClumpMeshRotations[ClumpIndex][MeshIndex] = MeshOrientation;
								}

								// Compute the clump mesh extents
								ClumpMeshExtentsX[ClumpIndex].Init(FVector::ZeroVector, NumMeshes + 1);
								ClumpMeshExtentsY[ClumpIndex].Init(FVector::ZeroVector, NumMeshes + 1);

								float MeshExtent = MinExtent;
								for (int32 MeshIndex = 0; MeshIndex < NumMeshes+1; ++MeshIndex)
								{
									MeshOrientation = (MeshIndex == NumMeshes) ? ClumpMeshRotations[ClumpIndex][MeshIndex-1] : ClumpMeshRotations[ClumpIndex][MeshIndex];

									const FVector MeshNormalX = MeshOrientation.GetAxisX();
									const FVector MeshNormalY = MeshOrientation.GetAxisY();

									if(MeshIndex == 0)
									{ 
										FVector2d MeshExtents(MinExtent, MinExtent);
										for (const FVector& MeshPoint : ClumpMeshPoints[ClumpIndex][MeshIndex])
										{
											const FVector LocalPoint = MeshPoint - ClumpMeshOrigins[ClumpIndex][MeshIndex];

											MeshExtents[0] = FMath::Max(MeshExtents[0], FMath::Abs(LocalPoint.Dot(MeshNormalX)));
											MeshExtents[1] = FMath::Max(MeshExtents[1], FMath::Abs(LocalPoint.Dot(MeshNormalY)));
										}
										MeshExtent = FMath::Min(MeshExtents[0], MeshExtents[1]);
									}
									ClumpMeshExtentsX[ClumpIndex][MeshIndex] = MeshNormalX * MeshExtent;
									ClumpMeshExtentsY[ClumpIndex][MeshIndex] = MeshNormalY * MeshExtent;
								}

								TArray<FVector3f> VertexLocalPositions;
								TArray<FVector3f> VertexLocalNormals;
								TArray<FIntVector> FaceLocalVertices;

								VertexLocalNormals.Init(FVector3f::ZeroVector, NumMeshes * NumCorners);
								VertexLocalPositions.Reserve(NumMeshes * NumCorners);
								FaceLocalVertices.Reserve(NumMeshes * NumTriangles);

								int32 VertexOffset = 0;

								auto AddMeshFace = [&VertexOffset, &FaceLocalVertices, &VertexLocalNormals, &VertexLocalPositions](const int32 VertexIndexA, const int32 VertexIndexB, const int32 VertexIndexC)
									{
										FIntVector& FaceIndices = FaceLocalVertices.Add_GetRef(FIntVector(VertexOffset + VertexIndexA, VertexOffset + VertexIndexB, VertexOffset + VertexIndexC));
										const FVector3f FaceNormal = ((VertexLocalPositions[FaceIndices[2]] - VertexLocalPositions[FaceIndices[0]]).Cross(VertexLocalPositions[FaceIndices[1]] - VertexLocalPositions[FaceIndices[0]])).GetSafeNormal();

										VertexLocalNormals[FaceIndices[0]] += FaceNormal;
										VertexLocalNormals[FaceIndices[1]] += FaceNormal;
										VertexLocalNormals[FaceIndices[2]] += FaceNormal;
									};
								for (int32 MeshIndex = 0; MeshIndex < NumMeshes; ++MeshIndex)
								{
									// Add vertex positions
									FVector3f MeshOrigin = FVector3f(ClumpMeshOrigins[ClumpIndex][MeshIndex]);
									FVector3f MeshExtentX = FVector3f(ClumpMeshExtentsX[ClumpIndex][MeshIndex]);
									FVector3f MeshExtentY = FVector3f(ClumpMeshExtentsY[ClumpIndex][MeshIndex]);

									VertexLocalPositions.Add(MeshOrigin - MeshExtentX - MeshExtentY);
									VertexLocalPositions.Add(MeshOrigin + MeshExtentX - MeshExtentY);
									VertexLocalPositions.Add(MeshOrigin + MeshExtentX + MeshExtentY);
									VertexLocalPositions.Add(MeshOrigin - MeshExtentX + MeshExtentY);

									MeshOrigin = FVector3f(ClumpMeshOrigins[ClumpIndex][MeshIndex+1]);
									MeshExtentX = FVector3f(ClumpMeshExtentsX[ClumpIndex][MeshIndex+1]);
									MeshExtentY = FVector3f(ClumpMeshExtentsY[ClumpIndex][MeshIndex+1]);

									VertexLocalPositions.Add(MeshOrigin - MeshExtentX - MeshExtentY);
									VertexLocalPositions.Add(MeshOrigin + MeshExtentX - MeshExtentY);
									VertexLocalPositions.Add(MeshOrigin + MeshExtentX + MeshExtentY);
									VertexLocalPositions.Add(MeshOrigin - MeshExtentX + MeshExtentY);

									// Add box faces and update normals
									AddMeshFace(0, 1, 2);
									AddMeshFace(2, 3, 0);
									AddMeshFace(7, 6, 5);
									AddMeshFace(5, 4, 7);
									AddMeshFace(3, 2, 6);
									AddMeshFace(6, 7, 3);
									AddMeshFace(0, 3, 7);
									AddMeshFace(7, 4, 0);
									AddMeshFace(2, 1, 5);
									AddMeshFace(5, 6, 2);
									AddMeshFace(1, 0, 4);
									AddMeshFace(4, 5, 1);

									for(int32 VertexIndex = 0; VertexIndex < NumCorners; ++VertexIndex)
									{
										VertexLocalNormals[VertexIndex + VertexOffset].Normalize();
									}
									VertexOffset += NumCorners;
								}

								const FLinearColor GroupColor = FLinearColor::IntToDistinctColor(ClumpIndex, 0.75f, 1.0f, 90.0f);

								TArray<FLinearColor> VertexLocalColors;
								VertexLocalColors.Init(GroupColor, VertexLocalPositions.Num());
								
								const FString GeometryName = TEXT("Groom_Clump_") + FString::FromInt(ClumpIndex);

								BuildGeometryCollection(VertexLocalPositions, VertexLocalNormals, FaceLocalVertices, VertexLocalColors, FName(GeometryName), DynamicMeshes[ClumpIndex]);
							}
							Instance.CacheValue(DynamicMeshes);
						}
					}
				}
			}
			AddMeshComponents(DynamicMeshes, OutComponents, FilterClumpIndices);
		}
	};

	class FGeometryCollectionCardsGeometryRenderableType : public Dataflow::IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FManagedArrayCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(CardsGeometry);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(UE::Dataflow::FDataflowConstruction3DViewMode);

		virtual bool CanRender(const Dataflow::FRenderableTypeInstance& Instance) const
		{
			const FManagedArrayCollection& Collection = GetCollection(Instance);
			return Collection.HasGroup(FGeometryCollection::GeometryGroup);
		}

		virtual void GetPrimitiveComponents(const Dataflow::FRenderableTypeInstance& Instance, Dataflow::FRenderableComponents& OutComponents) const override
		{
			TArray<Geometry::FDynamicMesh3> DynamicMeshes;
			if (Instance.HasUptoDateCachedValue())
			{
				DynamicMeshes = Instance.GetCachedValue<TArray<Geometry::FDynamicMesh3>>();
			}
			else
			{
				const FManagedArrayCollection& Collection = GetCollection(Instance);

				TArray<TArray<FVector3f>> VertexPositions, VertexNormals;
				TArray<TArray<FIntVector3>> FaceVertices;
				BuildCardsGeometry(Collection, VertexPositions, VertexNormals, FaceVertices);
				
				if(VertexPositions.Num() == VertexNormals.Num() && VertexNormals.Num() == FaceVertices.Num())
				{
					const int32 NumCards = VertexPositions.Num();

					DynamicMeshes.SetNum(NumCards);
					for(int32 CardIndex = 0; CardIndex < NumCards; ++CardIndex)
					{	
						const FLinearColor GroupColor = FLinearColor::IntToDistinctColor(CardIndex, 0.75f, 1.0f, 90.0f);

						TArray<FLinearColor> VertexColors;
						VertexColors.Init(GroupColor, VertexPositions[CardIndex].Num());

						const FString GeometryName = TEXT("Groom_Card_") + FString::FromInt(CardIndex);
					
						BuildGeometryCollection(VertexPositions[CardIndex], VertexNormals[CardIndex], FaceVertices[CardIndex], VertexColors, FName(GeometryName), DynamicMeshes[CardIndex]);
					}
					Instance.CacheValue(DynamicMeshes);
				}
			}
			AddMeshComponents(DynamicMeshes, OutComponents, {});
		}
	};

	class FGeometryCollectionCardsTextureRenderableType : public Dataflow::IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FManagedArrayCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(CardsTexture);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(UE::Dataflow::FDataflowConstruction3DViewMode);

		virtual bool CanRender(const Dataflow::FRenderableTypeInstance& Instance) const
		{
			const FManagedArrayCollection& Collection = GetCollection(Instance);
			return Collection.HasGroup(FGeometryCollection::GeometryGroup);
		}

		virtual void GetPrimitiveComponents(const Dataflow::FRenderableTypeInstance& Instance, Dataflow::FRenderableComponents& OutComponents) const override
		{
			TArray<Geometry::FDynamicMesh3> DynamicMeshes;
			TArray<TArray<int32>> FilterCardsIndices;
			if (Instance.HasUptoDateCachedValue())
			{
				DynamicMeshes = Instance.GetCachedValue<TArray<Geometry::FDynamicMesh3>>();
			}
			else
			{
				const FManagedArrayCollection& Collection = GetCollection(Instance);
				
				TArray<TArray<FVector3f>> VertexPositions, VertexNormals;
				TArray<TArray<FIntVector3>> FaceVertices;
				BuildCardsGeometry(Collection, VertexPositions, VertexNormals, FaceVertices);

				if(VertexPositions.Num() == VertexNormals.Num() && VertexNormals.Num() == FaceVertices.Num())
				{
					const int32 NumCards = VertexPositions.Num();

					FString CardsObjectsLODGroup = FGenerateCardsTexturesNode::CardsObjectsGroup.ToString();
					CardsObjectsLODGroup.AppendInt(GGroomDataflowCardsLod);

					if (Collection.HasAttribute(FGenerateCardsTexturesNode::ObjectTextureIndicesAttribute, FName(CardsObjectsLODGroup)))
					{
						const TManagedArray<int32>& CardsTextureArray = Collection.GetAttribute<int32>(
							FGenerateCardsTexturesNode::ObjectTextureIndicesAttribute, FName(CardsObjectsLODGroup));
				
						DynamicMeshes.SetNum(NumCards);
						
						TArray<int32> TextureIndices;
						TextureIndices.Init(INDEX_NONE, NumCards);
						FilterCardsIndices.SetNum(NumCards);
						
						int32 NumTextures = 0;
						for(int32 CardIndex = 0; CardIndex < NumCards; ++CardIndex)
						{
							if (CardsTextureArray.IsValidIndex(CardIndex) && CardsTextureArray[CardIndex] == CardIndex)
							{
								TextureIndices[CardIndex] = NumTextures;
								FilterCardsIndices[NumTextures].Add(CardIndex);
								++NumTextures;
							}
						}
						FilterCardsIndices.SetNum(NumTextures);
						
						for(int32 CardIndex = 0; CardIndex < NumCards; ++CardIndex)
						{
							const int32 TextureIndex = (CardIndex < CardsTextureArray.Num()) ? CardsTextureArray[CardIndex] : INDEX_NONE;
							FLinearColor CardColor = (TextureIndex != INDEX_NONE) ? FLinearColor::IntToDistinctColor(TextureIndex, 0.75f, 1.0f, 90.0f) : FLinearColor::Black;
							CardColor *= (TextureIndex == CardIndex) ? 1.0 : GGroomDataflowCardsAlpha;

							if(TextureIndex != CardIndex && TextureIndices.IsValidIndex(TextureIndex) && FilterCardsIndices.IsValidIndex(TextureIndices[TextureIndex]))
							{
								FilterCardsIndices[TextureIndices[TextureIndex]].Add(CardIndex);
							}

							TArray<FLinearColor> VertexColors;
							VertexColors.Init(CardColor, VertexPositions[CardIndex].Num());

							const FString GeometryName = TEXT("Groom_Texture_") + FString::FromInt(CardIndex);
							
							BuildGeometryCollection(VertexPositions[CardIndex], VertexNormals[CardIndex], FaceVertices[CardIndex], VertexColors, FName(GeometryName), DynamicMeshes[CardIndex]);
						}
						Instance.CacheValue(DynamicMeshes);
					}
				}
			}
			AddMeshComponents(DynamicMeshes, OutComponents, FilterCardsIndices);
		}
	};

	class FGeometryCollectionCardsUVsRenderableType : public Dataflow::IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FManagedArrayCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(CardsTexture);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(UE::Dataflow::FDataflowConstructionUVViewMode);

		virtual bool CanRender(const Dataflow::FRenderableTypeInstance& Instance) const
		{
			const FManagedArrayCollection& Collection = GetCollection(Instance);
			return Collection.HasGroup(FGeometryCollection::GeometryGroup);
		}

		virtual void GetPrimitiveComponents(const Dataflow::FRenderableTypeInstance& Instance, Dataflow::FRenderableComponents& OutComponents) const override
		{
			TArray<Geometry::FDynamicMesh3> DynamicMeshes;
			if (Instance.HasUptoDateCachedValue())
			{
				DynamicMeshes = Instance.GetCachedValue<TArray<Geometry::FDynamicMesh3>>();
			}
			else
			{
				const FManagedArrayCollection& Collection = GetCollection(Instance);
				
				FString CardsVerticesLODGroup = FGenerateCardsGeometryNode::CardsVerticesGroup.ToString();
				CardsVerticesLODGroup.AppendInt(GGroomDataflowCardsLod);
							
				FString CardsFacesLODGroup = FGenerateCardsGeometryNode::CardsFacesGroup.ToString();
				CardsFacesLODGroup.AppendInt(GGroomDataflowCardsLod);

				if (Collection.HasAttribute(FGenerateCardsTexturesNode::VertexTextureUVsAttribute, FName(CardsVerticesLODGroup)) &&
					Collection.HasAttribute(FGenerateCardsGeometryNode::FaceVertexIndicesAttribute, FName(CardsFacesLODGroup)))
				{
					const TManagedArray<FVector2f>& VertexUVsArray = Collection.GetAttribute<FVector2f>(
							FGenerateCardsTexturesNode::VertexTextureUVsAttribute, FName(CardsVerticesLODGroup));
						
					const TManagedArray<FIntVector3>& FaceIndicesArray = Collection.GetAttribute<FIntVector3>(
						FGenerateCardsGeometryNode::FaceVertexIndicesAttribute, FName(CardsFacesLODGroup));
					
					TArray<FIntVector> FaceIndices = FaceIndicesArray.GetConstArray();
					
					TArray<FVector3f> VertexUVs;
					VertexUVs.Init(FVector3f(0.0, 0.0, 1.0), VertexUVsArray.Num());

					for (int32 VertexIndex = 0; VertexIndex < VertexUVsArray.Num(); ++VertexIndex)
					{
						VertexUVs[VertexIndex] = FVector3f(VertexUVsArray[VertexIndex][0], VertexUVsArray[VertexIndex][1], 0.0f);
					}

					TArray<FVector3f> VertexNormals;
					VertexNormals.Init(FVector3f(0.0, 0.0, 1.0), VertexUVs.Num());

					TArray<FLinearColor> VertexColors;
					VertexColors.Init(FLinearColor(0,0,0,0), VertexUVs.Num());
					
					const FString GeometryName = TEXT("Groom_UVs");

					DynamicMeshes.SetNum(1);
					BuildGeometryCollection(VertexUVs, VertexNormals, FaceIndices, VertexColors, FName(GeometryName), DynamicMeshes.Last());
				}
			}
			AddMeshComponents(DynamicMeshes, OutComponents, {});
		}
	};

	void RegisterCollectionRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FGeometryCollectionCardsClumpsRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FGeometryCollectionCardsGeometryRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FGeometryCollectionCardsTextureRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FGeometryCollectionCardsUVsRenderableType);
	}
}

#undef LOCTEXT_NAMESPACE
