// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardGeneratorDataflowModule.h"
#include "GenerateCardsClumpsNode.h"
#include "BuildCardsSettingsNode.h"
#include "GenerateCardsGeometryNode.h"
#include "GenerateCardsTexturesNode.h"
#include "CardsAssetTerminalNode.h"
#include "ModifyCardsAttributesNode.h"
#include "HairCardDataflowRendering.h"
#include "Modules/ModuleManager.h"
#include "HairStrandsFactory.h"

#define LOCTEXT_NAMESPACE "HairCardGeneratorDataflow"

void FHairCardGeneratorDataflowModule::StartupModule()
{
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateCardsClumpsNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateCardsGeometryNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateCardsTexturesNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBuildCardsSettingsNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCardsAssetTerminalNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FExtractCardsAttributesNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FReportCardsAttributesNode);
	
	UE::CardGen::Private::RegisterCollectionRenderableTypes();
	UE::Groom::RegisterGroomDataflowTemplate({ "SelectCardsDataflowTemplate", "Cards", "Add a Dataflow with a cards generation.", "/HairCardGenerator/Dataflow/Templates/DF_GroomCardsTemplate.DF_GroomCardsTemplate", false });
}

void FHairCardGeneratorDataflowModule::ShutdownModule()
{
	UE::Groom::UnregisterGroomDataflowTemplate("SelectCardsDataflowTemplate");
}

IMPLEMENT_MODULE(FHairCardGeneratorDataflowModule, HairCardGeneratorDataflow)

#undef LOCTEXT_NAMESPACE
