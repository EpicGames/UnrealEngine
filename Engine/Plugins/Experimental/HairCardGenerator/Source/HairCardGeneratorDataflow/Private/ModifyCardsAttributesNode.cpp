// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModifyCardsAttributesNode.h"

#include "GroomBuilder.h"
#include "GroomInstance.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowTypePolicy.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModifyCardsAttributesNode)

namespace UE::Groom::Private
{
	static void ExtractGroomAttributes(FManagedArrayCollection& GroomCollection, const FHairDescription& HairDescription)
	{
		const GeometryCollection::Facades::FCollectionCurveGeometryFacade GeometryFacade(GroomCollection);
		GeometryCollection::Facades::FCollectionCurveAttributesFacade AttributesFacade(GroomCollection);
		
		const TStrandAttributesConstRef<FName> CardGroupsAttribute = HairDescription.StrandAttributes().GetAttributesRef<FName>(HairAttribute::Strand::GroupCardsName);
		
		if (AttributesFacade.IsValid() && GeometryFacade.IsValid())
		{
			FHairDescriptionGroups HairDescriptionGroups;
			FGroomBuilder::BuildHairDescriptionGroups(HairDescription, HairDescriptionGroups, false);
		
			const int32 NumCurves = GeometryFacade.GetNumCurves();
			
			TArray<FName> CardGroups;
			CardGroups.SetNum(NumCurves);
		
			for(int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
			{
				const int32 GroupIndex = GeometryFacade.GetCurveGeometryIndices()[CurveIndex];
				const int32 SourceCurve = GeometryFacade.GetCurveSourceIndices()[CurveIndex] >> 1;
			
				if (HairDescriptionGroups.HairGroups.IsValidIndex(GroupIndex) && CardGroups.IsValidIndex(CurveIndex) &&
					HairDescriptionGroups.HairGroups[GroupIndex].Strands.StrandsCurves.CurvesMapping.IsValidIndex(SourceCurve))
				{
					const int32 StrandID = HairDescriptionGroups.HairGroups[GroupIndex].Strands.StrandsCurves.CurvesMapping[SourceCurve];
					if (CardGroupsAttribute.IsValid() && (StrandID < CardGroupsAttribute.GetNumElements()) && (StrandID >= 0))
					{
						CardGroups[CurveIndex] = CardGroupsAttribute[FStrandID(StrandID)];
					}
					else
					{
						CardGroups[CurveIndex] = FName("Default");
					}
				}
			}
			AttributesFacade.SetCurveCardGroups(CardGroups);
		}
	}
	
	static void ReportGroomAttributes(const FManagedArrayCollection& GroomCollection, FHairDescription& HairDescription)
	{
		const GeometryCollection::Facades::FCollectionCurveGeometryFacade GeometryFacade(GroomCollection);
		const GeometryCollection::Facades::FCollectionCurveAttributesFacade AttributesFacade(GroomCollection);
		
		if (AttributesFacade.IsValid() && GeometryFacade.IsValid() && GeometryFacade.GetNumCurves() == AttributesFacade.GetNumCurves())
		{
			TStrandAttributesRef<FName> CardGroupsAttribute = HairDescription.StrandAttributes().GetAttributesRef<FName>(HairAttribute::Strand::GroupCardsName);
			if (!CardGroupsAttribute.IsValid())
			{
				HairDescription.StrandAttributes().RegisterAttribute<FName>(HairAttribute::Strand::GroupCardsName);
				CardGroupsAttribute = HairDescription.StrandAttributes().GetAttributesRef<FName>(HairAttribute::Strand::GroupCardsName);
			}
			
			FHairDescriptionGroups HairDescriptionGroups;
			FGroomBuilder::BuildHairDescriptionGroups(HairDescription, HairDescriptionGroups, false);
		
			const int32 NumCurves = GeometryFacade.GetNumCurves();
			const TArray<FName>& CardGroups = AttributesFacade.GetCurveCardGroups();
		
			for(int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
			{
				const int32 GroupIndex = GeometryFacade.GetCurveGeometryIndices()[CurveIndex];
				const int32 SourceCurve = GeometryFacade.GetCurveSourceIndices()[CurveIndex] >> 1;
			
				if (HairDescriptionGroups.HairGroups.IsValidIndex(GroupIndex) && CardGroups.IsValidIndex(CurveIndex) &&
					HairDescriptionGroups.HairGroups[GroupIndex].Strands.StrandsCurves.CurvesMapping.IsValidIndex(SourceCurve))
				{
					const int32 StrandID = HairDescriptionGroups.HairGroups[GroupIndex].Strands.StrandsCurves.CurvesMapping[SourceCurve];
					if ((StrandID < CardGroupsAttribute.GetNumElements()) && (StrandID >= 0))
					{
						CardGroupsAttribute[FStrandID(StrandID)] = CardGroups[CurveIndex];
					}
				}
			}
		}
	}
}

FExtractCardsAttributesNode::FExtractCardsAttributesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
		: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&GroomAsset);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&CardGroups);
}

void FExtractCardsAttributesNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection GroomCollection= GetValue<FManagedArrayCollection>(Context, &Collection);
		if(TObjectPtr<const UGroomAsset> InputAsset = GetValue<TObjectPtr<const UGroomAsset>>(Context, &GroomAsset))
		{
			const FHairDescription HairDesc = InputAsset->LoadHairDescription(EHairDescriptionType::Source);
			UE::Groom::Private::ExtractGroomAttributes(GroomCollection, HairDesc);
		}
		SetValue(Context, MoveTemp(GroomCollection), &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&CardGroups))
	{
		FCollectionAttributeKey AttributeKey;
		AttributeKey.Group = GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup.ToString();
		AttributeKey.Attribute = GeometryCollection::Facades::FCollectionCurveAttributesFacade::CurveCardGroupsAttribute.ToString();
		
		SetValue(Context, MoveTemp(AttributeKey), &CardGroups);
	}
}

FReportCardsAttributesNode::FReportCardsAttributesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
		: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&GroomAsset);
	RegisterOutputConnection(&Collection, &Collection);
}

void FReportCardsAttributesNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const FManagedArrayCollection& GroomCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		if(TObjectPtr<UGroomAsset> InputAsset = GetValue<TObjectPtr<UGroomAsset>>(Context, &GroomAsset))
		{
			FHairDescription HairDesc = InputAsset->LoadHairDescription(EHairDescriptionType::Source);
			UE::Groom::Private::ReportGroomAttributes(GroomCollection, HairDesc);
			
			InputAsset->CommitHairDescription(MoveTemp(HairDesc), EHairDescriptionType::Edit);
		}
		SetValue(Context, GroomCollection, &Collection);
	}
}


