// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "HairCardGeneratorPluginSettings.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "BuildCardsSettingsNode.generated.h"

/** Default parameters for hair cards generation */
namespace UE::HairCardsGen::Private
{
	static constexpr int32 NumClumps = 800;
	static constexpr int32 NumFlyaways = 100;
	static constexpr int32 NumTriangles = 45000;
	static constexpr int32 NumTextures = 100;
}

USTRUCT()
struct FGroomCardsSettings
{
	GENERATED_BODY()
	
	/** Generator settings to be built */
	UPROPERTY(EditAnywhere, Category = "Groom Cards");
	TObjectPtr<UHairCardGeneratorPluginSettings> GenerationSettings = nullptr;

	/** Generation flags to output the assets */
	UPROPERTY(EditAnywhere, Category = "Groom Cards");
	uint8 GenerationFlags = 0;

	/** Pipeline flags to generate clumps, geometry and textures */
	UPROPERTY(EditAnywhere, Category = "Groom Cards");
	uint8 PipelineFlags = 0;

	/** Groom asset to generate the */
    UPROPERTY(EditAnywhere, Category = "Groom Cards");
    TObjectPtr<UGroomAsset> GroomAsset = nullptr;
};


USTRUCT()
struct FGroomAdvancedFilterSettings
{
	GENERATED_BODY()
	
	// Generate multiple cards (3) per strand clump to give the appearance of volume
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Cards Filter")
	bool bUseMultiCardClumps = true;

	// Use adaptive subdivision when generating card geometry
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Cards Filter")
	bool bUseAdaptiveSubdivision = true;

	// Maximum number of segments along each card (aligned with the strands), ignored for adaptive subdivision
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Cards Filter")
	int32 MaxVerticalSegmentsPerCard = 10;

	// Scaling factor mapping strand-width to pixel-size for coverage texture
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Cards Filter")
	float StrandWidthScalingFactor = 1.0f;

	// Compress textures along strand direction to save vertical redolution,
	// if strands are all moving in nearly the same direction
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Cards Filter")
	bool UseOptimizedCompressionFactor = true;
};

USTRUCT()
struct FGroomFilterSettings
{
	GENERATED_BODY()

	// Filter group name to be identified in the override settings
	UPROPERTY(EditAnywhere, Category="Groom Cards")
	FName FilterName;

	// Total number of cards to generate for this LOD settings
	UPROPERTY(EditAnywhere, Category="Groom Cards", meta=(ClampMin="1", ClampMax="100000", DisplayName = "Num Clumps"))
	int32 NumClumps = UE::HairCardsGen::Private::NumClumps;

	// Maximum number of cards to assign to flyaway strands
	UPROPERTY(EditAnywhere, Category="Groom Cards", meta=(ClampMin="0", ClampMax="1000"))
	int32 NumFlyaways = UE::HairCardsGen::Private::NumFlyaways;

	// Total number of triangles to generate for this LOD settings
	UPROPERTY(EditAnywhere, Category="Groom Cards", meta=(ClampMin="1", ClampMax="100000"))
	int32 NumTriangles = UE::HairCardsGen::Private::NumTriangles;

	// Total number of textures to generate for this LOD settings
	UPROPERTY(EditAnywhere, Category="Groom Cards")
	int32 NumTextures = UE::HairCardsGen::Private::NumTextures;

	// Card group names that will belong to the filter group settings
	UPROPERTY(EditAnywhere, Category="Groom Cards")
	TArray<FName> CardGroups;

	// Advanced options for the filter group settings
	UPROPERTY(EditAnywhere, Category="Groom Cards")
	FGroomAdvancedFilterSettings AdvancedOptions;
};

USTRUCT()
struct FGroomAdvancedGenerationSettings
{
	GENERATED_BODY()
	
	// Use previous LOD generated cards and textures but reduce triangle count and flyaways
	UPROPERTY(EditAnywhere, AdvancedDisplay,Category="Cards Generation")
	bool bReduceCardsFromPreviousLOD = false;

	// Generate geometry for all groom groups on group 0
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Cards Generation", meta=(AllowPrivateAccess))
	bool bGenerateGeometryForAllGroups = true;

	// Seed value for pseudo-random number generation (set to a specific value for repeatable results)
	UPROPERTY(EditAnywhere, AdvancedDisplay,Category="Cards Generation", meta=(ClampMin="0", ClampMax="10000"))
	int32 RandomSeed = 0;

	// Place new card textures in reserved space from previous LOD
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Texture Rendering")
	bool bUseReservedSpaceFromPreviousLOD = false;

	// Size of hair card texture atlases
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Texture Rendering")
	EHairCardAtlasSize AtlasSize = EHairCardAtlasSize::AtlasSize4096;

	// Percentage of texture atlas space to reserve for higher LODs 
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Texture Rendering", meta=(ClampMin="0", ClampMax="75"))
	int32 ReserveTextureSpaceLOD = 0;

	// Use strand width
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Texture Rendering")
	bool bUseGroomAssetStrandWidth = true;
};

USTRUCT()
struct FGroomGenerationSettings
{
	GENERATED_BODY()
	
	// LOD index of the filter group settings
	UPROPERTY(EditAnywhere, Category="Groom Cards")
	int32 LODIndex = 0;

	// Group index of the filter group settings
	UPROPERTY(EditAnywhere, Category="Groom Cards")
	int32 GroupIndex = 0;

	// List of filter settings to use for that generation
	UPROPERTY(EditAnywhere, Category="Groom Cards")
	TArray<FGroomFilterSettings> FilterSettings;

	// Advanced options for the generation settings
	UPROPERTY(EditAnywhere, AdvancedDisplay,Category="Groom Cards")
	FGroomAdvancedGenerationSettings AdvancedOptions;
};

/** Build the cards generation settings */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FBuildCardsSettingsNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FBuildCardsSettingsNode, "BuildCardsSettings", "Groom", "")

public:
	
	FBuildCardsSettingsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override { return true; }
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override {return true;}
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
	virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End FDataflowNode interface

	/** Groom asset to build the cards settings from */
	UPROPERTY(EditAnywhere, Category = "Groom Cards", meta = (DataflowInput))
	TObjectPtr<UGroomAsset> GroomAsset;
	
	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;

	/** Generator cards settings to be built */
	UPROPERTY(meta = (DataflowOutput));
	TArray<FGroomCardsSettings> CardsSettings;
	
	/** List of filter settings to override */
	UPROPERTY(EditAnywhere, Category = "Groom Cards")
	TArray<FGroomGenerationSettings> GenerationSettings;
	
	// Assets directory path used when generating the mesh/textures
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Groom Cards")
	FString AssetsDirectoryPath = "";
	
	// Assets base name used when generating the mesh/textures
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Groom Cards")
	FString AssetsBaseName = "";
};

