// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "BuildCardsSettingsNode.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "CardsAssetTerminalNode.generated.h"

USTRUCT(meta = (Experimental, DataflowGroom, DataflowTerminal))
struct FCardsAssetTerminalNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCardsAssetTerminalNode, "CardsAssetTerminal", "Groom", "")

public:

	FCardsAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	
private:

	//~ Begin FDataflowTerminalNode interface
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowTerminalNode interface

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection", DataflowRenderGroups = "CardsTexture"))
	FManagedArrayCollection Collection;

	/** Generator settings to be used */
	UPROPERTY(meta = (DataflowInput, DisplayName = "CardsSettings"));
	TArray<FGroomCardsSettings> CardsSettings;
};

