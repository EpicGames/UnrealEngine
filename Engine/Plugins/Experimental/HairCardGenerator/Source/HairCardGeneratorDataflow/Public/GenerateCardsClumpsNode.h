// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "BuildCardsSettingsNode.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "GenerateCardsClumpsNode.generated.h"

USTRUCT()
struct FCardsClumpsSettings
{
	GENERATED_BODY()

	// Card filter name for which we will override the number of cards/flyaways
	UPROPERTY(EditAnywhere, Category="Groom Cards")
	FName FilterName = NAME_None;

	// Total number of cards to generate for this LOD settings
	UPROPERTY(EditAnywhere, Category="Groom Cards", meta=(ClampMin="1", ClampMax="100000", DisplayName = "Num Clumps"))
	int32 NumCards = UE::HairCardsGen::Private::NumClumps;

	// Maximum number of cards to assign to flyaway strands
	UPROPERTY(EditAnywhere, Category="Groom Cards", meta=(ClampMin="0", ClampMax="1000"))
	int32 NumFlyaways = UE::HairCardsGen::Private::NumFlyaways;
};

/** Generate the clumps used to build the cards from the strands */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FGenerateCardsClumpsNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateCardsClumpsNode, "GenerateCardsClumps", "Groom", "")

public:

	static const FName CurveClumpIndicesAttribute;
	static const FName ObjectNumClumpsAttribute;
	static const FName CurveFilterIndicesAttribute;
	
	FGenerateCardsClumpsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
	
	/** Per LOD settings for each cards generation */
	UPROPERTY(EditAnywhere, Category = "Groom Cards", meta = (DisplayName = "Override Settings"))
	TArray<FCardsClumpsSettings> ClumpsSettings;

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection", DataflowRenderGroups = "CardsClumps"))
	FManagedArrayCollection Collection;

	/** Generator settings to be used */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Cards Settings", DataflowPassthrough  = "CardsSettings"));
	TArray<FGroomCardsSettings> CardsSettings;
};

