// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "BuildCardsSettingsNode.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "GenerateCardsGeometryNode.generated.h"

USTRUCT()
struct FCardsGeometrySettings
{
	GENERATED_BODY()

	// Card filter name for which we will override the number of triangles
	UPROPERTY(EditAnywhere, Category="Groom Cards")
	FName FilterName = NAME_None;

	// Total number of triangles to generate for this LOD settings
	UPROPERTY(EditAnywhere, Category="Groom Cards", meta=(ClampMin="1", ClampMax="100000"))
	int32 NumTriangles = UE::HairCardsGen::Private::NumTriangles;
};

/** Generate the geometry used to build the cards from the strands */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FGenerateCardsGeometryNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateCardsGeometryNode, "GenerateCardsGeometry", "Groom", "")

public:

	static const FName VertexClumpPositionsAttribute;
	static const FName FaceVertexIndicesAttribute;
	static const FName VertexCardIndicesAttribute;
	
	static const FName CardsVerticesGroup;
	static const FName CardsFacesGroup;
	
	FGenerateCardsGeometryNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Per LOD settings for each cards generation */
	UPROPERTY(EditAnywhere, Category = "Groom Cards", meta = (DisplayName = "Override Settings"))
	TArray<FCardsGeometrySettings> GeometrySettings;

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection", DataflowRenderGroups = "CardsGeometry"))
	FManagedArrayCollection Collection;

	/** Generator settings to be used */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Cards Settings", DataflowPassthrough  = "CardsSettings"))
	TArray<FGroomCardsSettings> CardsSettings;
};

