// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "BuildCardsSettingsNode.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "GenerateCardsTexturesNode.generated.h"

USTRUCT()
struct FCardsTextureSettings
{
	GENERATED_BODY()

	// Card filter name for which we will override the number of textures
	UPROPERTY(EditAnywhere, Category="Groom Cards")
	FName FilterName = NAME_None;

	// Total number of textures to generate for this LOD settings
	UPROPERTY(EditAnywhere, Category="Groom Cards")
	int32 NumTextures = UE::HairCardsGen::Private::NumTextures;
};

/** Generate the cards texture for final rendering */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FGenerateCardsTexturesNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateCardsTexturesNode, "GenerateCardsTextures", "Groom", "")

public:


	static const FName CardsObjectsGroup;
	static const FName ObjectTextureIndicesAttribute;
	static const FName VertexTextureUVsAttribute;
	
	FGenerateCardsTexturesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Per LOD settings for each cards generation */
	UPROPERTY(EditAnywhere, Category = "Groom Cards", meta = (DisplayName = "Override Settings"))
	TArray<FCardsTextureSettings> TextureSettings;

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection", DataflowRenderGroups = "CardsTexture"))
	FManagedArrayCollection Collection;

	/** Generator settings to be used */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Cards Settings", DataflowPassthrough  = "CardsSettings"))
	TArray<FGroomCardsSettings> CardsSettings;
};

