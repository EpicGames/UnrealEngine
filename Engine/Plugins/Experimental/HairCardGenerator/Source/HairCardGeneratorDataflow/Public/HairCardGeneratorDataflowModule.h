// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/** Hair card generator dataflow module public interface */
class FHairCardGeneratorDataflowModule : public IModuleInterface
{
	public :
	
	//~ IModuleInterface begin interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ IModuleInterface end interface
};
