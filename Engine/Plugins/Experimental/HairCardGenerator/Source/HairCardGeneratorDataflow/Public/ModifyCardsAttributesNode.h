// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "BuildCardsSettingsNode.h"
#include "GetGroomAssetNode.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ModifyCardsAttributesNode.generated.h"

/** Extract the cards attributes from the description onto the collection */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FExtractCardsAttributesNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FExtractCardsAttributesNode, "ExtractCardsAttributes", "Groom", "")

public:
	
	FExtractCardsAttributesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
	
	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection", DataflowRenderGroups = "CardsClumps"))
	FManagedArrayCollection Collection;
	
	/** Input asset to read the attributes from */
	UPROPERTY(EditAnywhere, Category = "Groom", meta = (DataflowInput, DisplayName = "Groom Asset"))
	TObjectPtr<const UGroomAsset> GroomAsset = nullptr;
	
	/** Attribute key for the extracted card group  */
	UPROPERTY(EditAnywhere, Category = "Groom", meta = (DataflowOutput, DisplayName = "Card Groups"))
	FCollectionAttributeKey CardGroups;
};

/** Reports the cards attributes from the collection onto the description */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FReportCardsAttributesNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FReportCardsAttributesNode, "ReportCardsAttributes", "Groom", "")

public:
	
	FReportCardsAttributesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
	
	/** Managed array collection to be used to store data */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection", DataflowRenderGroups = "CardsClumps"))
	FManagedArrayCollection Collection;
	
	/** Input asset to write the attributes onto */
	UPROPERTY(EditAnywhere, Category = "Groom", meta = (DataflowInput, DisplayName = "Groom Asset"))
	TObjectPtr<UGroomAsset> GroomAsset = nullptr;
};


