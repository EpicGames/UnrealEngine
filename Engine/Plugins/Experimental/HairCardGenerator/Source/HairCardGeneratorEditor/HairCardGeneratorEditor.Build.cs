// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO; // for Path
public class HairCardGeneratorEditor : ModuleRules
{
	public HairCardGeneratorEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"HairCardGeneratorFramework",
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"HairStrandsCore", // for FHairDescription
				"MeshDescription",
				"StaticMeshDescription",
				"AssetRegistry",
				"UnrealEd",
				"Slate",
				"SlateCore",
				"EditorStyle",
				"MainFrame",
				"ApplicationCore",
				"InputCore",
				"ToolWidgets",
				"Eigen",
				"Json",
				"JsonUtilities",
				// ... add private dependencies that you statically link with here ...	
			}
		);
	}
}
