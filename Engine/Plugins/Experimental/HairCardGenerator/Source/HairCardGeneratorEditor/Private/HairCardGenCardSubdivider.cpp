// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardGenCardSubdivider.h"

FHairCardGenCardSubdivider::FHairCardGenCardSubdivider(const float SubdTolerance, const bool UseAdaptive, const int MaxSubdivisions):
	SubdToleranceInitial(SubdTolerance), UseAdaptive(UseAdaptive), MaxSubdivisions(MaxSubdivisions){}

void FHairCardGenCardSubdivider::SetSubdTolerance(const float NewSubdTolerance)
{
	SubdToleranceInitial = NewSubdTolerance;
}

MatrixXf FHairCardGenCardSubdivider::TArrayToMatrixXf(const TArray<float> Points)
{
	int N = Points.Num() / 3;
	MatrixXf PointMatrix(N, 3);

	for (int i = 0; i < N; i++) {
		for (int j = 0; j < 3; j++)
		{
			PointMatrix(i, j) = Points[3 * i + j];
		}
	}

	return PointMatrix;
}

TArray<float> FHairCardGenCardSubdivider::MatrixXfToTArray(const MatrixXf PointMatrix)
{
	int N = PointMatrix.rows();

	TArray<float> Points;
	Points.SetNum(3 * N);

	for (int i = 0; i < N; i++) {
		for (int j = 0; j < 3; j++)
		{
			Points[3 * i + j] = PointMatrix(i, j);
		}
	}

	return Points;
}

float FHairCardGenCardSubdivider::GetCurveLength(const MatrixXf Points)
{
	float TotalLength = 0.0;

	for (int i = 0; i < Points.rows() - 1; ++i)
	{
		TotalLength += (Points.row(i + 1) - Points.row(i)).norm();
	}

	return TotalLength;
}

float FHairCardGenCardSubdivider::GetAverageCurvatureRadius(const MatrixXf Points)
{
	int N = Points.rows();
	float AverageCurvatureRadius = 0.0;

	for(int i=0; i<(N - 2); ++i)
	{
		Vector3f P0 = Points.row(i);
		Vector3f P1 = Points.row(i + 1);
		Vector3f P2 = Points.row(i + 2);

		Vector3f T = P1 - P0;
		Vector3f U = P2 - P0;
		Vector3f V = P2 - P1;

		float TriangleArea = 2.0 * (T.cross(U)).norm() + 1e-10;

		AverageCurvatureRadius += T.norm() * U.norm() * V.norm() / TriangleArea;
	}

	return AverageCurvatureRadius / (N - 2);
}

int FHairCardGenCardSubdivider::NextSubdPoint(const int IndStart)
{
	int IndEnd = IndStart + 1;
	float CurveDistance = 0;
	float LinearDistance = 0;

	while (IndEnd < NumInterpPoints - 1)
	{
		LinearDistance = (MeanCurve.row(IndEnd) - MeanCurve.row(IndStart)).norm();
		CurveDistance += (MeanCurve.row(IndEnd) - MeanCurve.row(IndEnd - 1)).norm();

		if (CurveDistance - LinearDistance > SubdTolerance)
		{
			return IndEnd;
		}
		IndEnd++;
	}

	LastDist = CurveDistance - LinearDistance;
	return -1;
}

TArray<float> FHairCardGenCardSubdivider::AdaptiveIter()
{
	TArray<float> SubdPoints;
	SubdPoints.Reserve(60);

	int Ind = 0;

	do {
		for (float x : MeanCurve.row(Ind)) SubdPoints.Add(x);
		Ind = NextSubdPoint(Ind);
	} while (Ind > 0);

	for (float x : MeanCurve.row(NumInterpPoints - 1)) SubdPoints.Add(x);

	return SubdPoints;
}

TArray<float> FHairCardGenCardSubdivider::GetAdaptiveSubdPoints()
{
	TArray<float> SubdPointsNew, SubdPoints = AdaptiveIter();
	int NumPoints = SubdPoints.Num() / 3;

	LastDist = 0.;

	while (LastDist < 0.9 * SubdTolerance)
	{
		SubdTolerance *= 0.95f;
		SubdPointsNew = AdaptiveIter();
		if (SubdPointsNew.Num() / 3 != NumPoints)
		{
			return SubdPoints;
		}
		SubdPoints = SubdPointsNew;
	}

	return SubdPoints;
}

MatrixXf FHairCardGenCardSubdivider::LinearInterpolation(const MatrixXf Points, const int NumPoints)
{
	MatrixXf InterpMat(NumPoints, 3);

	int InputSize = Points.rows();

	for (int j = 0; j < 3; j++)
	{
		InterpMat(0, j) = Points(0, j);
		InterpMat(NumPoints - 1, j) = Points(InputSize - 1, j);

		for (int i = 1; i < NumPoints - 1; i++){
			float IndOutF = (float)(i * (InputSize - 1)) / (float)(NumPoints - 1);

			if (fabs(ceilf(IndOutF) - IndOutF) < 1e-5f)
			{
				InterpMat(i, j) = Points((int)ceil(IndOutF), j);
			}
			else
			{
				int IndOutI = (int)floor(IndOutF);
				InterpMat(i, j) = Points(IndOutI, j) + (Points(IndOutI + 1, j) - Points(IndOutI, j)) * (IndOutF - (float)IndOutI);
			}
		}
	}

	return InterpMat;
}

TArray<float> FHairCardGenCardSubdivider::GetSubdivisionPoints(const MatrixXf Points)
{
	TArray<float> SubdPoints;

	if (UseAdaptive)
	{
		if (SubdToleranceInitial > 0.)
		{
			SubdTolerance = SubdToleranceInitial;

			MeanCurve = LinearInterpolation(Points, NumInterpPoints);
			SubdPoints = GetAdaptiveSubdPoints();
		}
		else
		{
			SubdPoints = MatrixXfToTArray(LinearInterpolation(Points, 2));
		}
	}
	else
	{
		SubdPoints = MatrixXfToTArray(LinearInterpolation(Points, MaxSubdivisions + 1));
	}

	return SubdPoints;
}

float FHairCardGenCardSubdivider::GetToleranceFromLengthAndCurvRadius(float Length, float CurvRadius, float Subdivisions)
{
	if (Subdivisions <= 0.5) return -1.;

	float Num = Length * Length * Length;
	float Den = 3. * CurvRadius * CurvRadius * (2. * Subdivisions - 1.) * (2. * Subdivisions - 1.) * (2. * Subdivisions - 1.);

	return Num / Den;
}