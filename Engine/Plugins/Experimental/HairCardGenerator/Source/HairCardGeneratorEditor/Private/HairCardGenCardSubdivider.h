// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifndef EIGEN_MPL2_ONLY
#define EIGEN_MPL2_ONLY
#endif

#include "Containers/Array.h"

PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Dense>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END

using Eigen::MatrixXf;
using Eigen::Vector3f;

class FHairCardGenCardSubdivider
{
private:
	float SubdTolerance;
	float SubdToleranceInitial;
	const bool UseAdaptive;
	const int MaxSubdivisions;
	const int NumInterpPoints = 1000;
	float LastDist;

	MatrixXf MeanCurve;

private:
	static TArray<float> MatrixXfToTArray(const MatrixXf PointMatrix);
	int NextSubdPoint(const int IndStart);
	TArray<float> AdaptiveIter();
	TArray<float> GetAdaptiveSubdPoints();
	static MatrixXf LinearInterpolation(const MatrixXf, const int NumPoints);

public:
	FHairCardGenCardSubdivider(const float SubdTolerance, const bool UseAdaptive, const int MaxSubdivisions);
	void SetSubdTolerance(const float NewSubdTolerance);
	static MatrixXf TArrayToMatrixXf(const TArray<float> Points);
	TArray<float> GetSubdivisionPoints(const MatrixXf Points);
	static float GetCurveLength(const MatrixXf Points);
	static float GetAverageCurvatureRadius(const MatrixXf Points);
	static float GetToleranceFromLengthAndCurvRadius(const float Length, const float CurvRadius, const float Subdivisions);
};