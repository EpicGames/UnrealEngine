// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardGenControllerBase.h"


#include "GroomAsset.h"
#include "MaterialDomain.h"
#include "StaticMeshAttributes.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"

#include "HairCardGeneratorPluginSettings.h"
#include "HairCardGeneratorLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HairCardGenControllerBase)

UHairCardGenControllerBase::UHairCardGenControllerBase()
{
}

void UHairCardGenControllerBase::CreateCardsStaticMesh(UStaticMesh* StaticMesh, const TArray<float>& verts, const TArray<int32>& faces, const TArray<float>& normals, const TArray<float>& uvs, const TArray<int32>& groups)
{	
	// Set up a mapping so that each card gets its own polygon group id
	int MaxGroup = 0;
	for ( int GroupIdx : groups)
		MaxGroup = FMath::Max(MaxGroup, GroupIdx);

	// Set up basic model build settings
	StaticMesh->SetNumSourceModels(1);
	FStaticMeshSourceModel& Model = StaticMesh->GetSourceModel(0);
	Model.BuildSettings.bRecomputeNormals = false;
	Model.BuildSettings.bRecomputeTangents = true;
	Model.BuildSettings.bComputeWeightedNormals = true;
	Model.BuildSettings.bGenerateLightmapUVs = false;
	Model.BuildSettings.bRemoveDegenerates = true;

	FMeshDescription* MeshDescription = StaticMesh->CreateMeshDescription(0);
	FStaticMeshAttributes MeshAttributes(*MeshDescription);

	// Add a default material slot to the mesh
	FName MaterialName = FName(TEXT("Material_0"));
	FStaticMaterial DefaultMaterial;
	DefaultMaterial.MaterialInterface = UMaterial::GetDefaultMaterial(EMaterialDomain::MD_Surface);
	DefaultMaterial.MaterialSlotName = MaterialName;
	DefaultMaterial.ImportedMaterialSlotName = MaterialName;

	StaticMesh->GetStaticMaterials().Add(DefaultMaterial);

	TVertexAttributesRef<FVector3f> VertexPositions = MeshAttributes.GetVertexPositions();
	TPolygonGroupAttributesRef<FName> PolygonGroupMaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = MeshAttributes.GetVertexInstanceUVs();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = MeshAttributes.GetVertexInstanceNormals();


	// Add each unique group index to map and create a polygon group (also assign all groups default material)
	TMap<int32,FPolygonGroupID> CardToGroupMap;
	CardToGroupMap.Reserve(MaxGroup+1);
	for ( int32 VertIndex = 0; VertIndex < groups.Num(); ++VertIndex )
	{
		// int GroupIndex = groups[VertIndex];
		int GroupIndex = 0;
		if ( CardToGroupMap.Contains(GroupIndex) )
			continue;

		const FPolygonGroupID PolygonGroupID = MeshDescription->CreatePolygonGroup();
		PolygonGroupMaterialSlotNames[PolygonGroupID] = MaterialName;

		CardToGroupMap.Add(GroupIndex, PolygonGroupID);
	}

	ensure(verts.Num() % 3 == 0);
	for ( int VertIndex = 0; VertIndex < verts.Num()/3; ++VertIndex )
	{
		FVertexID VertexID = MeshDescription->CreateVertex();
		VertexPositions[VertexID] = FVector3f(verts[3*VertIndex], verts[3*VertIndex+1], verts[3*VertIndex+2]);
	}

	for ( int TriIdx = 0; TriIdx < faces.Num() / 3; ++TriIdx )
	{
		TArray<FVertexInstanceID> TriVertIDs;
		TriVertIDs.SetNum(3);

		// Convert from counter-clockwise winding to clockwise (also flip precomputed normals)
		const int32 VOrder[] = {2,1,0};
		for ( int cidx = 0; cidx < 3; ++cidx )
		{
			const int32 TriCornerIdx = 3*TriIdx + VOrder[cidx];
			const int32 VertCornerIdx = faces[TriCornerIdx];

			const FVertexID VertexID(VertCornerIdx);
			const FVertexInstanceID VertexInstanceID = MeshDescription->CreateVertexInstance(VertexID);

			VertexInstanceUVs.Set(VertexInstanceID, 0, FVector2f(uvs[2*VertCornerIdx],uvs[2*VertCornerIdx+1]));
			VertexInstanceNormals[VertexInstanceID] = -FVector3f(normals[3*VertCornerIdx],normals[3*VertCornerIdx+1],normals[3*VertCornerIdx+2]);

			TriVertIDs[cidx] = VertexInstanceID;
		}

		// int GroupIndex = groups[faces[3*TriIdx]];
		int GroupIndex = 0;
		const FPolygonGroupID& PolygonGroupID = CardToGroupMap.FindRef(GroupIndex);

		MeshDescription->CreatePolygon(PolygonGroupID, TriVertIDs);
	}

	StaticMesh->CommitMeshDescription(0);
}

TObjectPtr<UHairCardGeneratorPluginSettings>& UHairCardGenControllerBase::GetGroomSettings(TObjectPtr<UGroomAsset> Groom, int LODIndex)
{
	const FString SettingsTag = Groom->GetPathName() + TEXT(":LOD") + FString::FromInt(LODIndex);

	// NOTE: Only keep around one settings object per-lod
	return GroomSettingsMap.FindOrAdd(SettingsTag);
}

void UHairCardGenControllerBase::UpdateGroomSettings(TObjectPtr<UGroomAsset> Groom, int LODIndex, int GroupID, TObjectPtr<UHairCardGeneratorPluginSettings> NewSettings)
{
	const FString SettingsTag = Groom->GetPathName() + TEXT(":LOD") + FString::FromInt(LODIndex);
	GroomSettingsMap.FindOrAdd(SettingsTag) = NewSettings;
}
