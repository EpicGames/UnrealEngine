// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "GroomCacheData.h"
#include "Containers/Map.h"
#include "Templates/Tuple.h"
#include "HairCardGenTypes.h"

#include "HairCardGenControllerBase.generated.h"

class FHairDescription;
//struct FHairCardGenerationData;

USTRUCT(BlueprintType)
struct FHairCardGen_StrandData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Strand")
	int GroupID = -1;
};

// With "Guide" data stipped out (since it isn't used as part of the mesh generation algorithm)
USTRUCT(BlueprintType)
struct FHairCardGen_GroomData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Groom")
	FString BasisType;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Groom")
	FString CurveType;

	// Array of index buffers. Each buffer defines a distinct hair strand in the groom.
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Groom")
	TArray<FHairCardGen_StrandData> Strands;

	// Flattened Array of 3D vetex positions 
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Groom")
	TArray<float> VertexPositions;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Groom")
	TArray<float> VertexWidths;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Groom")
	TArray<float> VertexOcclusions;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Groom")
	int32 HairlineGroupID = INDEX_NONE;
};


class UGroomAsset;
class UHairCardGeneratorPluginSettings;

/**
 * 
 */
UCLASS(Blueprintable)
class UHairCardGenControllerBase : public UObject
{
	GENERATED_BODY()

public:
	UHairCardGenControllerBase();

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	int GetPointsPerCurve();

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool LoadGroomData(const FHairCardGen_GroomData& GroomData, const FString& Name, const FString& CachedGroomsPath, const bool SaveCached = false);

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool LoadSettings(const UHairCardGeneratorPluginSettings* GeneratorSettings);

	// ---- Per-group stage ufunctions ----

	/** Pure compute. Returns clump assignment via OutClumpData (bool-first + pure-out). */
	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool GenerateClumps(int32 GroupId, FHairCardClumpData& OutClumpData);

	/** Pure compute. Builds in-memory geometry_opt state from clump data. */
	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool SetOptimizations(int32 GroupId, const FHairCardClumpData& ClumpData);

	/** Pure compute. Returns card geometry via OutGeomData (bool-first + pure-out). */
	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool GenerateCardsGeometry(FHairCardGeomData& OutGeomData, FHairCardMeshData& OutMeshData, int32& OutCardCount);

	/** Pure compute. Returns texture cluster assignment via OutClusterData (bool-first + pure-out). */
	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool ClusterTextures(int32 GroupId, const FHairCardGeomData& GeomData, const FHairCardMeshData& MeshData,
	                     FHairCardTextureClusterData& OutClusterData);

	// ---- Cross-group stage ufunctions ----

	/** Pure compute. Receives all-group data; returns atlas layout and per-vertex atlas UVs (bool-first + pure-out). */
	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool GenerateTextureLayout(const TArray<FHairCardGeomData>& AllGeomData,
	                           const TArray<FHairCardMeshData>& AllMeshData,
	                           const TArray<FHairCardTextureClusterData>& AllClusterData,
	                           FHairCardAtlasLayoutData& OutLayoutData,
	                           FHairCardAtlasBinpackData& OutBinpackData,
	                           FHairCardAtlasUVData& OutAtlasUVData);

	/** Pure compute. Renders texture atlases to disk. */
	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool GenerateTextureAtlases(const TArray<FHairCardGeomData>& AllGeomData,
	                            const TArray<FHairCardMeshData>& AllMeshData,
	                            const TArray<FHairCardTextureClusterData>& AllClusterData,
	                            const FHairCardAtlasLayoutData& LayoutData,
								const FHairCardAtlasBinpackData& BinpackData,
	                            const float WidthScale = -1);

	/** Pure compute. Writes the static mesh asset. */
	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool GenerateMesh(UStaticMesh* StaticMesh,
	                  const TArray<FHairCardGeomData>& AllGeomData,
	                  const TArray<FHairCardMeshData>& AllMeshData,
	                  const TArray<FHairCardTextureClusterData>& AllClusterData,
	                  const FHairCardAtlasLayoutData& LayoutData);

	// ---- Geometry generation internal functions ----

	/** Returns num sub-cards per clump optimizer; available after SetOptimizations. */
	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	TArray<int32> GetNumCardsPerClump();

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	TArray<float> GetAverageCurve(const int Id, const int Cid);

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	void SetInterpolatedAvgCurve(const int Id, const int Cid, const TArray<float>& Points);

	// ---- Save / Load ufunctions ----

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool SaveClumpData(int32 GroupId, const FHairCardClumpData& ClumpData);

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool LoadClumpData(int32 GroupId, FHairCardClumpData& OutClumpData);

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool SaveCardData(int32 GroupId, const FHairCardGeomData& GeometryData, const FHairCardMeshData& MeshData, int32 CardCount);

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool LoadCardData(int32 GroupId, FHairCardGeomData& OutGeomData, FHairCardMeshData& OutMeshData, int32& OutCardCount);

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool SaveTextureClusterData(int32 GroupId, const FHairCardTextureClusterData& TexClusterData);

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool LoadTextureClusterData(int32 GroupId, FHairCardTextureClusterData& OutTexClusterData);

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool SaveAtlasLayoutData(const FHairCardAtlasLayoutData& LayoutData);

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool LoadAtlasLayoutData(FHairCardAtlasLayoutData& OutLayoutData);

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool SaveAtlasBinpackData(const FHairCardAtlasBinpackData& BinpackData);

	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	bool LoadAtlasBinpackData(FHairCardAtlasBinpackData& OutBinpackData);

	// Helper C++ function for building a static mesh from verts/faces, etc.
	UFUNCTION(BlueprintCallable, Category = Python)
	void CreateCardsStaticMesh(UStaticMesh* StaticMesh, const TArray<float>& verts, const TArray<int32>& faces, const TArray<float>& normals, const TArray<float>& uvs, const TArray<int32>& groups);


	TObjectPtr<UHairCardGeneratorPluginSettings>& GetGroomSettings(TObjectPtr<UGroomAsset> Groom, int LODIndex);
	void UpdateGroomSettings(TObjectPtr<UGroomAsset> Groom, int LODIndex, int GroupID, TObjectPtr<UHairCardGeneratorPluginSettings> NewSettings);

private:


	UPROPERTY(Transient)
	TMap<FString,TObjectPtr<UHairCardGeneratorPluginSettings>> GroomSettingsMap;
};
