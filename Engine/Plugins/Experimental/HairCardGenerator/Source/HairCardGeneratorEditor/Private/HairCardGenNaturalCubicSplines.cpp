// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardGenNaturalCubicSplines.h"

FHairCardGenNaturalCubicSplines::FHairCardGenNaturalCubicSplines(const TArray<float>& Knots, const TArray<float>& Values)
{
    MakeSpline(Knots, Values);
    MakeBins();
}

float FHairCardGenNaturalCubicSplines::operator()(float t) const
{
    SplineData const& Spline = FindSpline(t);

    double Value = Spline.Coefficients[3];
    for (int i = 2; i >= 0; i--) {
        Value *= t - Spline.Knot;
        Value += Spline.Coefficients[i];
    }

    return Value;
}

void FHairCardGenNaturalCubicSplines::MakeSpline(const TArray<float>& Knots, const TArray<float>& Values)
{
    const int NumSegments = Knots.Num() - 1;

    TArray<float> Intervals; Intervals.SetNum(NumSegments);
    TArray<float> Slopes; Slopes.SetNum(NumSegments);

    for (int i = 0; i < NumSegments; i++)
    {
        const float dt = Knots[i + 1] - Knots[i];
        const float dx = Values[i + 1] - Values[i];

        Intervals[i] = dt;
        Slopes[i] = dx / dt;
    }

    TArray<float> L; L.SetNum(NumSegments + 1);
    TArray<float> D; D.SetNum(NumSegments + 1);
    TArray<float> U; U.SetNum(NumSegments + 1);
    TArray<float> Y; Y.SetNum(NumSegments + 1);

    D[0] = 1;
    D[NumSegments] = 1;

    for (int i = 1; i < NumSegments; i++)
    {
        L[i] = Intervals[i - 1];
        D[i] = 2 * (Intervals[i - 1] + Intervals[i]);
        U[i] = Intervals[i];
        Y[i] = 6 * (Slopes[i] - Slopes[i - 1]);
    }

    SolveTridiagonalSystem(L, D, U, Y);
    TArray<float> const& M = Y;

    Splines.Empty();
    Splines.SetNum(NumSegments);

    for (int i = 0; i < NumSegments; i++) {
        SplineData Spline;
        Spline.Knot = Knots[i];
        Spline.Coefficients[0] = Values[i];
        Spline.Coefficients[1] = Slopes[i] - (M[i + 1] + 2 * M[i]) * Intervals[i] / 6;
        Spline.Coefficients[2] = M[i] / 2;
        Spline.Coefficients[3] = (M[i + 1] - M[i]) / (6 * Intervals[i]);
        Splines[i] = Spline;
    }

    LowerBound = Knots[0];
    UpperBound = Knots[NumSegments];
}

void FHairCardGenNaturalCubicSplines::MakeBins()
{
    BinInterval = (UpperBound - LowerBound) / (float)Splines.Num();

    int Index = 0;

    for (int Bin = 0; ; Bin++)
    {
        const float BinEdge = LowerBound + BinInterval * Bin;

        while (Index + 1 < Splines.Num() && Splines[Index + 1].Knot <= BinEdge)
        {
            Index++;
        }

        Bins.Add(Index);

        if (Index + 1 == Splines.Num())
        {
            break;
        }
    }

    Bins.Shrink();
}

FHairCardGenNaturalCubicSplines::SplineData const& FHairCardGenNaturalCubicSplines::FindSpline(float t) const
{
    if (t <= LowerBound)
    {
        return Splines[0];
    }
    if (t >= UpperBound)
    {
        return Splines.Last();
    }

    int Bin = int((t - LowerBound) / BinInterval);
    if (Bin >= Bins.Num())
    {
        Bin = Bins.Num() - 1;
    }

    int Index = Bins[Bin];
    for (; Index + 1 < Splines.Num(); Index++)
    {
        if (t < Splines[Index + 1].Knot)
        {
            break;
        }
    }

    return Splines[Index];
}

void FHairCardGenNaturalCubicSplines::SolveTridiagonalSystem(TArray<float>& Lower, TArray<float>& Diag, TArray<float>& Upper, TArray<float>& RHS)
{
    const int Dim = RHS.Num();

    for(int i = 0; i < Dim - 1; i++)
    {
        if (abs(Diag[i]) >= abs(Lower[i + 1]))
        {
            const float W = Lower[i + 1] / Diag[i];
            Diag[i + 1] -= W * Upper[i];
            RHS[i + 1] -= W * RHS[i];
            Lower[i + 1] = 0;
        }
        else
        {
            const float W = Diag[i] / Lower[i + 1];
            const float U = Diag[i + 1];

            Diag[i] = Lower[i + 1];
            Diag[i + 1] = Upper[i] - W * U;
            Lower[i + 1] = Upper[i + 1];
            Upper[i + 1] *= -W;
            Upper[i] = U;

            const float R = RHS[i];
            RHS[i] = RHS[i + 1];
            RHS[i + 1] = R - W * RHS[i + 1];
        }
    }

    RHS[Dim - 1] /= Diag[Dim - 1];

    for (int iRev = 2; iRev <= Dim; iRev++)
    {
        int i = Dim - iRev;
        if (i == Dim - 2)
        {
            RHS[i] -= Upper[i] * RHS[i + 1];
        }
        else
        {
            RHS[i] -= Upper[i] * RHS[i + 1] + Lower[i + 1] * RHS[i + 2];
        }
        RHS[i] /= Diag[i];
    }
}