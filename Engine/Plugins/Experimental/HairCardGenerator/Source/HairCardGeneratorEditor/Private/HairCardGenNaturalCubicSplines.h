// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

class FHairCardGenNaturalCubicSplines
{
private:
    struct SplineData
    {
        double Knot;
        double Coefficients[4];
    };

    TArray<SplineData> Splines;
    TArray<int> Bins;
    float LowerBound;
    float UpperBound;
    float BinInterval;

private:
    void MakeSpline(const TArray<float>& Knots, const TArray<float>& Values);
    void MakeBins();
    SplineData const& FindSpline(float t) const;
    void SolveTridiagonalSystem(TArray<float>& Lower, TArray<float>& Diag, TArray<float>& Upper, TArray<float>& RHS);

public:
	FHairCardGenNaturalCubicSplines(const TArray<float>& Knots, const TArray<float>& Values);
    float operator()(float t) const;
};