// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardGenOptionsWindow.h"
#include "HairCardGeneratorPluginSettings.h"

#include "GroomAsset.h"
#include "GroomAssetCards.h"
#include "IDetailsView.h"
#include "IDocumentation.h"
#include "InputCoreTypes.h"
#include "PropertyEditorModule.h"
#include "SPrimaryButton.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Input/Reply.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/PackageName.h"
#include "Styling/AppStyle.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#include "HairCardGeneratorLog.h"

#define LOCTEXT_NAMESPACE "HairCardSettings"

/* SGroupSettingsListRow Declaration
 *****************************************************************************/

class SHairCardGenOptionsWindow;

class SGroupSettingsListRow : public SMultiColumnTableRow< TObjectPtr<UHairCardGeneratorGroupSettings> >
{
public:
	SLATE_BEGIN_ARGS( SGroupSettingsListRow ){}
	SLATE_END_ARGS()
public:
	void Construct( const FArguments& InArgs, TObjectPtr<UHairCardGeneratorGroupSettings> InGroupSettings, const TSharedRef<STableViewBase>& InOwnerTableView );

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	FText TestGetIndex() const;

	// Card group filter widget
	FText GetGroupFilterText() const;

	// Number of cards widget
	int32 ReadTargetNumCards() const;
	void WriteTargetNumCards(int NumCards);

	// Number of textures widget
	int32 ReadTargetNumTextures() const;
	void WriteTargetNumTextures(int NumTextures);

	// Number of flyaway cards widget
	int32 ReadMaxFlyaways() const;
	void WriteMaxFlyaways(int MaxFlyaways);

	// Number of target triangles widget
	bool CheckEditableTriangles() const;
	int32 ReadTargetNumTriangles() const;
	void WriteTargetNumTriangles(int NumTris);

	// Strand count widget
	FText GetStrandCountText() const;

	// Settings group generation state widget
	FText GetGeneratedStateText() const;

	// Remove Settings group (button widget)
	FReply DeleteSettingsGroup();

	// Return false if reduce from previous LOD is selected
	bool CheckDisabledOnReducePrevLOD() const;

	int32 GetIntPropertyMetadata(FName PropertyName, FName MetadataField, int32 DefaultValue = 0) const;

private:
	TObjectPtr<UHairCardGeneratorGroupSettings> GroupSettingsRow;
};


/* SHairCardGenOptionsWindow Declaration
 *****************************************************************************/

class SHairCardGenOptionsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SHairCardGenOptionsWindow)
		: _SettingsObject(nullptr)
		, _WidgetWindow()
		, _MaxWindowHeight(0.0f)
		, _MaxWindowWidth(0.0f)
	{}

		SLATE_ARGUMENT(TObjectPtr<UHairCardGeneratorPluginSettings>, SettingsObject)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(float, MaxWindowHeight)
		SLATE_ARGUMENT(float, MaxWindowWidth)
	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs);

	//~ Begin SWidget interface
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget interface

	SHairCardGenOptionsWindow()
		: SettingsObject(nullptr)
		, bRunOnClose(false)
	{}

	bool RunGenerationOnClose() const { return bRunOnClose; }

private:
	EActiveTimerReturnType SetFocusPostConstruct(double InCurrentTime, float InDeltaTime);
	bool CanRunGeneration() const;
	bool CanOpenAdvancedSettings() const;

	FReply OnAddSettingsGroup();
	FReply OnEditClick();
	FReply OnGenerateClick();
	FReply OnCancelClick();
	FReply OnResetToDefaultClick() const;

	FText GetTargetAssetName() const;

	const FSlateBrush* GetErrorIcon() const;
	EVisibility GetErrorIconVisibility() const;
	FText GetErrorIconTooltip() const;
	FText GetGenerateButtonTooltip() const;

	void OnEditGroupSettings(TObjectPtr<UHairCardGeneratorGroupSettings> GroupSettings);

	ECheckBoxState IsForceRegen() const;
	void OnForceRegenToggle(ECheckBoxState NewState);

	ESelectionMode::Type GetListSelectionMode() const { return ESelectionMode::Single; }
	TSharedRef<ITableRow> OnGenerateOptionListRow(TObjectPtr<UHairCardGeneratorGroupSettings> InGroupSettings, const TSharedRef<STableViewBase>& OwnerTable);

private:
	// Main settings object
	TObjectPtr<UHairCardGeneratorPluginSettings> SettingsObject;
	// Should run (not cancelled)
	bool bRunOnClose = false;

	// Main window
	TWeakPtr<SWindow> WidgetWindow;
	// Generation settings details pane
	TSharedPtr<class IDetailsView> DetailsView;
	// List view for updating per settings-group options
	TSharedPtr<SListView<TObjectPtr<UHairCardGeneratorGroupSettings>>> GroupSettingsListView;
	TSharedPtr<SButton> RunButton;
};


/* SHairCardGroupSettingsWindow Declaration
 *****************************************************************************/

class SHairCardGroupSettingsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SHairCardGroupSettingsWindow)
		: _FilterGroupIndex(0)
		, _SettingsObject(nullptr)
		, _WidgetWindow()
		, _MaxWindowHeight(0.0f)
		, _MaxWindowWidth(0.0f)
	{}

		SLATE_ARGUMENT(int, FilterGroupIndex)
		SLATE_ARGUMENT(TObjectPtr<UHairCardGeneratorPluginSettings>, SettingsObject)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(float, MaxWindowHeight)
		SLATE_ARGUMENT(float, MaxWindowWidth)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	//~ Begin SWidget interface
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget interface

	SHairCardGroupSettingsWindow()
		: SettingsObject(nullptr)
	{}

private:
	EActiveTimerReturnType SetFocusPostConstruct(double InCurrentTime, float InDeltaTime);
	FReply OnOkClick();
	FReply OnResetToDefaultClick() const;
	FText GetIntermediateTargetName() const;

private:
	int FilterGroupIndex;
	TObjectPtr<UHairCardGeneratorPluginSettings> SettingsObject;

	// Main window
	TWeakPtr<SWindow> WidgetWindow;
	// Group settings detail view
	TSharedPtr<IDetailsView> DetailsView;
	TSharedPtr<SButton> OkButton;
};


/* HairCardGenWindow_Impl Declaration
 *****************************************************************************/

namespace HairCardGenWindow_Impl
{
	static bool IsValidPath(const FDirectoryPath& ContentPath, const FString& PkgName);
	static bool DoFilesExist(const FDirectoryPath& ContentPath, const FString& PkgName);

	static bool PromptGroupSettingsDialog(TSharedPtr<SWindow> ParentWindow, TObjectPtr<UHairCardGeneratorPluginSettings> Settings, int FilterGroupIndex);
}


/* HairCardGenWindow_Utils Definition
 *****************************************************************************/

bool HairCardGenWindow_Utils::PromptUserWithHairCardGenDialog(UHairCardGeneratorPluginSettings* SettingsObject)
{
	TSharedPtr<SWindow> ParentWindow;
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	// Always set force regen to false on window open
	// SettingsObject->SetForceRegenerate(false);

	// Compute centered window position based on max window size, which include when all categories are expanded
	const float WindowWidth = 600.0f;
	const float WindowHeight = 400.0f;
	FVector2D WindowSize = FVector2D(WindowWidth, WindowHeight); // Max window size it can get based on current slate

	FSlateRect WorkAreaRect = FSlateApplicationBase::Get().GetPreferredWorkArea();
	FVector2D DisplayTopLeft(WorkAreaRect.Left, WorkAreaRect.Top);
	FVector2D DisplaySize(WorkAreaRect.Right - WorkAreaRect.Left, WorkAreaRect.Bottom - WorkAreaRect.Top);

	float ScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(DisplayTopLeft.X, DisplayTopLeft.Y);
	WindowSize *= ScaleFactor;

	FVector2D WindowPosition = (DisplayTopLeft + (DisplaySize - WindowSize) / 2.0f) / ScaleFactor;

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("HairCardSettings.Title", "Card Generation Settings"))
		.SizingRule(ESizingRule::UserSized)
		.AutoCenter(EAutoCenter::None)
		.ClientSize(WindowSize)
		.ScreenPosition(WindowPosition);

	TSharedPtr<SHairCardGenOptionsWindow> HairCardGenWindow;
	Window->SetContent
	(
		SAssignNew(HairCardGenWindow, SHairCardGenOptionsWindow)
			.SettingsObject(SettingsObject)
			.WidgetWindow(Window)
	);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, /*bSlowTaskWindow =*/false);

	return HairCardGenWindow->RunGenerationOnClose();
}


/* SGroupSettingsListRow Declaration
 *****************************************************************************/
void SGroupSettingsListRow::Construct( const FArguments& InArgs, TObjectPtr<UHairCardGeneratorGroupSettings> InGroupSettings, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	GroupSettingsRow = InGroupSettings;
	FSuperRowType::Construct( FSuperRowType::FArguments().Padding(0), InOwnerTableView);
}

TSharedRef<SWidget> SGroupSettingsListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if ( ColumnName == TEXT("SettingsIndex") )
	{
		return SNew(STextBlock)
			.Text(this, &SGroupSettingsListRow::TestGetIndex);
	}
	else if ( ColumnName == TEXT("CardGroupFilter") )
	{
		return SNew(STextBlock)
			.Text(this, &SGroupSettingsListRow::GetGroupFilterText)
			.ToolTipText(LOCTEXT("HairCardSettings.CardGroupFilter.ToolTip", "The card groups to which these settings apply"));
	}
	else if ( ColumnName == TEXT("TargetNumCards") )
	{
		return SNew(SSpinBox<int>)
			.MinValue(GetIntPropertyMetadata(GET_MEMBER_NAME_CHECKED(UHairCardGeneratorGroupSettings, TargetNumberOfCards), TEXT("ClampMin"), 1))
			.MaxValue(GetIntPropertyMetadata(GET_MEMBER_NAME_CHECKED(UHairCardGeneratorGroupSettings, TargetNumberOfCards), TEXT("ClampMax"), 10000))
			.Value(this, &SGroupSettingsListRow::ReadTargetNumCards)
			.OnValueChanged(this, &SGroupSettingsListRow::WriteTargetNumCards)
			.IsEnabled(this, &SGroupSettingsListRow::CheckDisabledOnReducePrevLOD)
			.ToolTipText(LOCTEXT("HairCardSettings.TargetNumCards.ToolTip", "The number of cards to generate"));
	}
	else if ( ColumnName == TEXT("TargetNumTextures") )
	{
		return SNew(SSpinBox<int>)
			.MinValue(GetIntPropertyMetadata(GET_MEMBER_NAME_CHECKED(UHairCardGeneratorGroupSettings, NumberOfTexturesInAtlas), TEXT("ClampMin"), 1))
			.MaxValue(GetIntPropertyMetadata(GET_MEMBER_NAME_CHECKED(UHairCardGeneratorGroupSettings, NumberOfTexturesInAtlas), TEXT("ClampMax"), 300))
			.Value(this, &SGroupSettingsListRow::ReadTargetNumTextures)
			.OnValueChanged(this, &SGroupSettingsListRow::WriteTargetNumTextures)
			.IsEnabled(this, &SGroupSettingsListRow::CheckDisabledOnReducePrevLOD)
			.ToolTipText(LOCTEXT("HairCardSettings.TargetNumTextures.ToolTip.InvalidTextureNum", "The number of textures to generate (must be <= number of cards)"));
	}
	else if ( ColumnName == TEXT("MaxFlyaways") )
	{
		return SNew(SSpinBox<int>)
			.MinValue(GetIntPropertyMetadata(GET_MEMBER_NAME_CHECKED(UHairCardGeneratorGroupSettings, MaxNumberOfFlyaways), TEXT("ClampMin"), 0))
			.MaxValue(GetIntPropertyMetadata(GET_MEMBER_NAME_CHECKED(UHairCardGeneratorGroupSettings, MaxNumberOfFlyaways), TEXT("ClampMax"), 1000))
			.Value(this, &SGroupSettingsListRow::ReadMaxFlyaways)
			.OnValueChanged(this, &SGroupSettingsListRow::WriteMaxFlyaways)
			.ToolTipText(LOCTEXT("HairCardSettings.MaxFlyaways.ToolTip", "The maximum number of flyaway cards (single distinct strand cards) to allow"));
	}
	else if ( ColumnName == TEXT("TargetNumTriangles") )
	{
		return SNew(SSpinBox<int>)
			.MinValue(GetIntPropertyMetadata(GET_MEMBER_NAME_CHECKED(UHairCardGeneratorGroupSettings, TargetTriangleCount), TEXT("ClampMin"), 2))
			.MaxValue(GetIntPropertyMetadata(GET_MEMBER_NAME_CHECKED(UHairCardGeneratorGroupSettings, TargetTriangleCount), TEXT("ClampMax"), 100000))
			.Value(this, &SGroupSettingsListRow::ReadTargetNumTriangles)
			.OnValueChanged(this, &SGroupSettingsListRow::WriteTargetNumTriangles)
			.IsEnabled(this, &SGroupSettingsListRow::CheckEditableTriangles)
			.ToolTipText(LOCTEXT("HairCardSettings.TargetTriangles.ToolTip", "The total number of triangles to use for all cards (approximate)"));
	}
	else if ( ColumnName == TEXT("NumStrands") )
	{
		return SNew(STextBlock)
			.Text(this, &SGroupSettingsListRow::GetStrandCountText)
			.ToolTipText(LOCTEXT("HairCardSettings.NumStrands.ToolTip", "The number of strands these settings will apply to"));
	}
	else if ( ColumnName == TEXT("IsGenerated") )
	{
		return SNew(STextBlock)
			.Text(this, &SGroupSettingsListRow::GetGeneratedStateText)
			.ToolTipText(LOCTEXT("HairCardSettings.IsGenerated.ToolTip", "Indicates whether cards and textures have already been generated using these settings"));
	}
	else if ( ColumnName == TEXT("AddDeleteRow") )
	{
		return SNew(SButton)
			.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
			.ToolTipText(LOCTEXT("HairCardSettings.RemoveGroup.ToolTip", "Remove this settings group"))
			.OnClicked( this, &SGroupSettingsListRow::DeleteSettingsGroup )
			// .ForegroundColor(FSlateColor::UseForeground())
			.HAlign( HAlign_Center )
			.VAlign( VAlign_Center )
			.Content()
			[
				SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
			];
	}

	return
		SNew(STextBlock)
		. Text( FText::Format( LOCTEXT("HairCardSettings.UnsupprtedColumn", "Unsupported Column: {0}"), FText::FromName( ColumnName ) ) );
}

FText SGroupSettingsListRow::TestGetIndex() const
{
	// NOTE: Indices can be wrong for a tick after Construct is called so currently just used as a label.
	return FText::Format(LOCTEXT("HairCardSettings.SettingsIndex", "{0}"), FText::AsNumber(IndexInList));
}

FText SGroupSettingsListRow::GetGroupFilterText() const
{
	TStringBuilder<256> ListBuilder;
	ListBuilder.AppendChar(TCHAR('['));
	for ( FName id : GroupSettingsRow->ApplyToCardGroups )
	{
		ListBuilder.Appendf(TEXT("%s, "), *id.ToString());
	}
	if ( GroupSettingsRow->ApplyToCardGroups.Num() > 0 )
	{
		ListBuilder.RemoveSuffix(2);
	}
	ListBuilder.AppendChar(TCHAR(']'));

	return FText::Format(LOCTEXT("HairCardSettings.CardGroupFilter", "{0}"), FText::FromString(FString(ListBuilder)));
}

int32 SGroupSettingsListRow::ReadTargetNumCards() const
{
	return GroupSettingsRow->TargetNumberOfCards;
}

void SGroupSettingsListRow::WriteTargetNumCards(int NumCards)
{
	GroupSettingsRow->TargetNumberOfCards = NumCards;
}

int32 SGroupSettingsListRow::ReadTargetNumTextures() const
{
	return GroupSettingsRow->NumberOfTexturesInAtlas;
}

void SGroupSettingsListRow::WriteTargetNumTextures(int NumTextures)
{
	GroupSettingsRow->NumberOfTexturesInAtlas = NumTextures;
}

int32 SGroupSettingsListRow::ReadMaxFlyaways() const
{
	return GroupSettingsRow->MaxNumberOfFlyaways;
}

void SGroupSettingsListRow::WriteMaxFlyaways(int MaxFlyaways)
{
	GroupSettingsRow->MaxNumberOfFlyaways = MaxFlyaways;
}

bool SGroupSettingsListRow::CheckEditableTriangles() const
{
	return GroupSettingsRow->UseAdaptiveSubdivision;
}

int32 SGroupSettingsListRow::ReadTargetNumTriangles() const
{
	return GroupSettingsRow->TargetTriangleCount;
}

void SGroupSettingsListRow::WriteTargetNumTriangles(int NumTris)
{
	GroupSettingsRow->TargetTriangleCount = NumTris;
}

FText SGroupSettingsListRow::GetStrandCountText() const
{
	return FText::Format(LOCTEXT("HairCardSettings.GroupStrandCount", "{0}"), FText::AsNumber(GroupSettingsRow->GetStrandCount()));
}

FText SGroupSettingsListRow::GetGeneratedStateText() const
{
	// TODO: Return specific generation string or split this field
	uint8 NeedGenerateFlags = GroupSettingsRow->GetFilterGroupGeneratedFlags();
	if ( NeedGenerateFlags == (uint8)EHairCardGenerationPipeline::None )
		return FText::Format(LOCTEXT("HairCardSettings.GroupGenerate", "{0}"), FText::FromString(TEXT("No")));

	if ( NeedGenerateFlags == (uint8)EHairCardGenerationPipeline::All )
		return FText::Format(LOCTEXT("HairCardSettings.GroupGenerate", "{0}"), FText::FromString(TEXT("Yes")));

	return  FText::Format(LOCTEXT("HairCardSettings.GroupGenerate", "{0}"), FText::FromString(TEXT("Partial")));
}

FReply SGroupSettingsListRow::DeleteSettingsGroup()
{
	GroupSettingsRow->RemoveSettingsFilterGroup();
	return FReply::Handled();
}

bool SGroupSettingsListRow::CheckDisabledOnReducePrevLOD() const
{
	// Disable editing if reducing from previous LOD
	return !GroupSettingsRow->GetReduceFromPreviousLOD();
}

int32 SGroupSettingsListRow::GetIntPropertyMetadata(FName PropertyName, FName MetadataField, int32 DefaultValue) const
{
	for (TFieldIterator<FProperty> PropertyIt(GroupSettingsRow->GetClass()); PropertyIt; ++PropertyIt)
	{
		if (PropertyIt->GetFName() != PropertyName)
		{
			continue;
		}

		if ( !PropertyIt->HasMetaData(MetadataField) )
		{
			return DefaultValue;
		}

		return  PropertyIt->GetIntMetaData(MetadataField);
	}

	return DefaultValue;
}


/* SHairCardGenOptionsWindow Definition
 *****************************************************************************/

void SHairCardGenOptionsWindow::Construct(const FArguments& InArgs)
{
	SettingsObject = InArgs._SettingsObject;
	WidgetWindow = InArgs._WidgetWindow;

	check(SettingsObject);

	TSharedPtr<SBox> HeaderBox;
	TSharedPtr<SBox> DetailsBox;
	this->ChildSlot
	[
		SNew(SScrollBox)
		+SScrollBox::Slot()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
			[
				SAssignNew(HeaderBox, SBox)
			]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
			[
				SAssignNew(DetailsBox, SBox)
					.MaxDesiredHeight(500.0f)
					.WidthOverride(700.0f)
			]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
			[
				SAssignNew(GroupSettingsListView, SListView<TObjectPtr<UHairCardGeneratorGroupSettings>>)
					.ListItemsSource(&SettingsObject->GetFilterGroupSettings())
					.OnGenerateRow(this, &SHairCardGenOptionsWindow::OnGenerateOptionListRow)
					.SelectionMode(this, &SHairCardGenOptionsWindow::GetListSelectionMode)
					.OnMouseButtonDoubleClick(this, &SHairCardGenOptionsWindow::OnEditGroupSettings)
					.HeaderRow(
						SNew(SHeaderRow)
							+ SHeaderRow::Column("SettingsIndex").DefaultLabel(LOCTEXT("HairCardSettings.SettingsIndex.Header", "Id")).FixedWidth(30)
							+ SHeaderRow::Column("CardGroupFilter").DefaultLabel(LOCTEXT("HairCardSettings.CardGroupFilter.Header", "Card Groups"))
							+ SHeaderRow::Column("TargetNumCards").DefaultLabel(LOCTEXT("HairCardSettings.TargetNumCards.Header", "# Cards"))
							+ SHeaderRow::Column("TargetNumTextures").DefaultLabel(LOCTEXT("HairCardSettings.TargetNumTextures.Header", "# Textures"))
							+ SHeaderRow::Column("TargetNumTriangles").DefaultLabel(LOCTEXT("HairCardSettings.TargetNumTriangles.Header", "# Triangles"))
							+ SHeaderRow::Column("MaxFlyaways").DefaultLabel(LOCTEXT("HairCardSettings.MaxFlyaways.Header", "Max Flyaway Cards"))
							+ SHeaderRow::Column("NumStrands").DefaultLabel(LOCTEXT("HairCardSettings.StrandCount.Header", "Strand Count"))
							+ SHeaderRow::Column("IsGenerated").DefaultLabel(LOCTEXT("HairCardSettings.GroupGenerate.Header", "Generate"))
							+ SHeaderRow::Column("AddDeleteRow").FixedWidth(50)
							[
								SNew(SButton)
									.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
									.OnClicked(this, &SHairCardGenOptionsWindow::OnAddSettingsGroup)
									.ToolTipText(LOCTEXT("HairCardSettings.AddSettingsGroup.ToolTip", "Add new settings group"))
									.HAlign( HAlign_Center )
									.VAlign( VAlign_Center )
									[
										SNew(SImage)
											.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
									]
							]
					)
			]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
			[
				SNew(SUniformGridPanel)
					.SlotPadding(1)
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("HairCardSettings.GroupEditSettings", "Advanced..."))
						.ToolTipText(LOCTEXT("HairCardSettings.GroupEditSettings.ToolTip", "Edit advanced settings for currently selected hair card group."))
						.OnClicked(this, &SHairCardGenOptionsWindow::OnEditClick)
						.IsEnabled(this, &SHairCardGenOptionsWindow::CanOpenAdvancedSettings)
				]
				+ SUniformGridPanel::Slot(2, 0)
				[
					SNew(SCheckBox)
					.HAlign(HAlign_Right)
					.ToolTipText(LOCTEXT("HairCardSettings.ForceRegen.ToolTip", "Force full regeneration"))
					.IsChecked(this, &SHairCardGenOptionsWindow::IsForceRegen)
					.OnCheckStateChanged(this,  &SHairCardGenOptionsWindow::OnForceRegenToggle)
					[
						SNew(STextBlock).Text(LOCTEXT("HairCardSettings.ForceRegen", "Force Regenerate"))
					]
				]
				+ SUniformGridPanel::Slot(3, 0)
				[
					SAssignNew(RunButton, SPrimaryButton)
						.Text(LOCTEXT("HairCardSettings.Generate", "Generate"))
						.ToolTipText(this, &SHairCardGenOptionsWindow::GetGenerateButtonTooltip)
						.OnClicked(this, &SHairCardGenOptionsWindow::OnGenerateClick)
						.IsEnabled(this, &SHairCardGenOptionsWindow::CanRunGeneration)
				]
				+ SUniformGridPanel::Slot(4, 0)
				[
					SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("HairCardSettings.Cancel", "Cancel"))
						.ToolTipText(LOCTEXT("HairCardSettings.Cancel.ToolTip", "Cancel hair card generation."))
						.OnClicked(this, &SHairCardGenOptionsWindow::OnCancelClick)
				]
			]
		]
	];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailsBox->SetContent(DetailsView->AsShared());

	HeaderBox->SetContent(
		SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FMargin(2.0f, 0.0f))
			[
				SNew(SImage)
					.Image(this, &SHairCardGenOptionsWindow::GetErrorIcon)
					.ToolTipText(this, &SHairCardGenOptionsWindow::GetErrorIconTooltip)
					.Visibility(this, &SHairCardGenOptionsWindow::GetErrorIconVisibility)
			]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(this, &SHairCardGenOptionsWindow::GetTargetAssetName)
			]
			+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(FMargin(2.0f, 0.0f))
			[
				SNew(SButton)
					.Text(LOCTEXT("HairCardSettings.ResetOptions", "Reset hair card solver settings to defaults"))
					.OnClicked(this, &SHairCardGenOptionsWindow::OnResetToDefaultClick)
			]
		]
	);
	DetailsView->SetObject(SettingsObject);
	GroupSettingsListView->RequestListRefresh();

	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SHairCardGenOptionsWindow::SetFocusPostConstruct));
}


bool SHairCardGenOptionsWindow::CanRunGeneration() const
{
	TMap<uint32,uint32> ErrorCountMap;
	SettingsObject->ValidateStrandFilters(ErrorCountMap);

	if ( ErrorCountMap.Num() > 0 )
		return false;

	return HairCardGenWindow_Impl::IsValidPath(SettingsObject->DestinationPath, SettingsObject->BaseFilename);
}

bool SHairCardGenOptionsWindow::CanOpenAdvancedSettings() const
{
	int SelectCount = GroupSettingsListView->GetNumItemsSelected();
	return !SettingsObject->bReduceCardsFromPreviousLOD && (SelectCount == 1);
}

FReply SHairCardGenOptionsWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCancelClick();
	}
	else if ( InKeyEvent.GetKey() == EKeys::Enter && CanRunGeneration() )
	{
		return OnGenerateClick();
	}

	return FReply::Unhandled();
}

EActiveTimerReturnType SHairCardGenOptionsWindow::SetFocusPostConstruct(double InCurrentTime, float InDeltaTime)
{
	if (RunButton.IsValid())
	{
		FSlateApplication::Get().SetKeyboardFocus(RunButton, EFocusCause::SetDirectly);
	}

	return EActiveTimerReturnType::Stop;
}

FReply SHairCardGenOptionsWindow::OnAddSettingsGroup()
{
	SettingsObject->AddNewSettingsFilterGroup();
	return FReply::Handled();
}

FReply SHairCardGenOptionsWindow::OnEditClick()
{
	TArray<TObjectPtr<UHairCardGeneratorGroupSettings>> selectedItems;
	int count = GroupSettingsListView->GetSelectedItems(selectedItems);
	if ( count == 1 )
		OnEditGroupSettings(selectedItems[0]);

	return FReply::Handled();
}

FReply SHairCardGenOptionsWindow::OnCancelClick()
{
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SHairCardGenOptionsWindow::OnResetToDefaultClick() const
{
	SettingsObject->ResetToDefault(FString(), FString());
	DetailsView->SetObject(SettingsObject, true);
	return FReply::Handled();
}

FReply SHairCardGenOptionsWindow::OnGenerateClick()
{
	bRunOnClose = true;
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

void SHairCardGenOptionsWindow::OnEditGroupSettings(TObjectPtr<UHairCardGeneratorGroupSettings> FilterGroupSettings)
{
	if ( FilterGroupSettings == nullptr || !CanOpenAdvancedSettings() )
		return;

	int FilterGroupIndex = SettingsObject->GetFilterGroupSettings().Find(FilterGroupSettings);
	if ( FilterGroupIndex == INDEX_NONE )
		return;

	HairCardGenWindow_Impl::PromptGroupSettingsDialog(WidgetWindow.Pin(), SettingsObject, FilterGroupIndex);
}

ECheckBoxState SHairCardGenOptionsWindow::IsForceRegen() const
{
	return (SettingsObject->GetForceRegenerate()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SHairCardGenOptionsWindow::OnForceRegenToggle(ECheckBoxState NewState)
{
	SettingsObject->SetForceRegenerate(NewState == ECheckBoxState::Checked);
}

FText SHairCardGenOptionsWindow::GetTargetAssetName() const
{
	return FText::FromString(SettingsObject->BaseFilename);
}

const FSlateBrush* SHairCardGenOptionsWindow::GetErrorIcon() const
{
	TMap<uint32,uint32> ErrorCountMap;
	SettingsObject->ValidateStrandFilters(ErrorCountMap);
	if ( ErrorCountMap.Num() > 0 )
		return FAppStyle::Get().GetBrush("Icons.ErrorWithColor");

	if (!HairCardGenWindow_Impl::IsValidPath(SettingsObject->DestinationPath, SettingsObject->BaseFilename))
		return FAppStyle::Get().GetBrush("Icons.ErrorWithColor");

	return FAppStyle::Get().GetBrush("Icons.WarningWithColor");
}

EVisibility SHairCardGenOptionsWindow::GetErrorIconVisibility() const
{
	TMap<uint32,uint32> ErrorCountMap;
	bool HasUnassignedStrands = SettingsObject->ValidateStrandFilters(ErrorCountMap);
	if ( ErrorCountMap.Num() > 0 || HasUnassignedStrands )
	{
		return EVisibility::Visible;
	}

	if (!HairCardGenWindow_Impl::IsValidPath(SettingsObject->DestinationPath, SettingsObject->BaseFilename))
	{
		return EVisibility::Visible;
	}

	return HairCardGenWindow_Impl::DoFilesExist(SettingsObject->DestinationPath, SettingsObject->BaseFilename) ?
		EVisibility::Visible : EVisibility::Collapsed;
}

FText SHairCardGenOptionsWindow::GetErrorIconTooltip() const
{
	TMap<uint32,uint32> ErrorCountMap;
	bool HasUnassignedStrands = SettingsObject->ValidateStrandFilters(ErrorCountMap);
	if ( ErrorCountMap.Num() > 0 )
	{
		return LOCTEXT("HairCardSettings.Generate.MultiAssignedStrandsError", "Some strands are included in more than one settings group, cannot run generation");
	}
	else if ( HasUnassignedStrands )
	{
		return LOCTEXT("HairCardSettings.Generate.UnassignedStrandsWarning", "The settings groups do not cover all groom strands, some strands will be left out of generation.");
	}

	if (!HairCardGenWindow_Impl::IsValidPath(SettingsObject->DestinationPath, SettingsObject->BaseFilename))
	{
		return LOCTEXT("HairCardSettings.Generate.InvalidFilenameWarning", "The specified filepath is not valid.");
	}
	return LOCTEXT("HairCardSettings.Generate.OverwriteFileWarning", "There are pre-existing files using this name. They are likely the previous card assets, which will be replaced and overwritten by this.");
}

FText SHairCardGenOptionsWindow::GetGenerateButtonTooltip() const
{
	TMap<uint32,uint32> ErrorCountMap;
	bool HasUnassignedStrands = SettingsObject->ValidateStrandFilters(ErrorCountMap);
	if ( ErrorCountMap.Num() > 0 )
	{
		return LOCTEXT("HairCardSettings.Generate.MultiAssignedStrandsError.ToolTip", "ERROR: Some strands are included in more than one settings group, cannot run generation");
	}
	else if ( HasUnassignedStrands )
	{
		return LOCTEXT("HairCardSettings.Generate.UnassignedStrandsWarning.ToolTip", "Warning: The settings groups do not cover all groom strands, some strands will be left out of generation");
	}

	return LOCTEXT("HairCardSettings.Generate.ToolTip", "Generate hair cards for all card groups. Will replace current card assets and textures on sucessful run.");
}

TSharedRef<ITableRow> SHairCardGenOptionsWindow::OnGenerateOptionListRow(TObjectPtr<UHairCardGeneratorGroupSettings> InGroupSettings, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SGroupSettingsListRow, InGroupSettings, OwnerTable);
}


/* SHairCardGroupSettingsWindow Definition
 *****************************************************************************/

void SHairCardGroupSettingsWindow::Construct(const FArguments& InArgs)
{
	FilterGroupIndex = InArgs._FilterGroupIndex;
	SettingsObject = InArgs._SettingsObject;
	WidgetWindow = InArgs._WidgetWindow;

	check(SettingsObject);
	check(SettingsObject->GetFilterGroupSettings().Num() > FilterGroupIndex);

	TSharedPtr<SBox> HeaderBox;
	TSharedPtr<SBox> DetailsBox;
	this->ChildSlot
	[
		SNew(SBox)
			.MaxDesiredHeight(InArgs._MaxWindowHeight)
			.MaxDesiredWidth(InArgs._MaxWindowWidth)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
			[
				SAssignNew(HeaderBox, SBox)
			]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
			[
				SAssignNew(DetailsBox, SBox)
					.MaxDesiredHeight(700.0f)
					.WidthOverride(400.0f)
			]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
			[
				SNew(SUniformGridPanel)
					.SlotPadding(2)
				+ SUniformGridPanel::Slot(1, 0)
				[
					SAssignNew(OkButton, SPrimaryButton)
						.Text(LOCTEXT("HairCardSettings.Group.Ok", "Ok"))
						.ToolTipText(LOCTEXT("HairCardSettings.Group.Ok.ToolTip", "Save hair card generation settings for this card group."))
						.OnClicked(this, &SHairCardGroupSettingsWindow::OnOkClick)
				]
				// + SUniformGridPanel::Slot(2, 0)
				// [
				// 	SNew(SButton)
				// 		.HAlign(HAlign_Center)
				// 		.Text(LOCTEXT("HairCardSettings.Group.Cancel", "Cancel"))
				// 		.ToolTipText(LOCTEXT("HairCardSettings.Group.Cancel.ToolTip", "Discards generation settings changes for this card group."))
				// 		.OnClicked(this, &SHairCardGroupGenOptionsWindow::OnCancelClick)
				// ]
			]
		]
	];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailsBox->SetContent(DetailsView->AsShared());

	HeaderBox->SetContent(
		SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(this, &SHairCardGroupSettingsWindow::GetIntermediateTargetName)
			]
			+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.Padding(FMargin(2.0f, 0.0f))
			[
				SNew(SButton)
					.Text(LOCTEXT("HairCardGenWindow_ResetOptions", "Reset to Default"))
					.OnClicked(this, &SHairCardGroupSettingsWindow::OnResetToDefaultClick)
			]
		]
	);
	DetailsView->SetObject(SettingsObject->GetFilterGroupSettings()[FilterGroupIndex]);

	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SHairCardGroupSettingsWindow::SetFocusPostConstruct));
}

FReply SHairCardGroupSettingsWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// TODO: Support cancel/ok in details UI
	if (InKeyEvent.GetKey() == EKeys::Escape || InKeyEvent.GetKey() == EKeys::Enter)
	{
		return OnOkClick();
	}

	return FReply::Unhandled();
}

FText SHairCardGroupSettingsWindow::GetIntermediateTargetName() const
{
	return FText::FromString(SettingsObject->BaseFilename + TEXT("_GIDX") + FString::FromInt(FilterGroupIndex));
}

EActiveTimerReturnType SHairCardGroupSettingsWindow::SetFocusPostConstruct(double InCurrentTime, float InDeltaTime)
{
	if (OkButton.IsValid())
	{
		FSlateApplication::Get().SetKeyboardFocus(OkButton, EFocusCause::SetDirectly);
	}

	return EActiveTimerReturnType::Stop;
}

FReply SHairCardGroupSettingsWindow::OnOkClick()
{
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SHairCardGroupSettingsWindow::OnResetToDefaultClick() const
{
	SettingsObject->ResetFilterGroupSettingsToDefault(FilterGroupIndex);
	//Refresh the view to make sure the custom UI are updating correctly
	DetailsView->SetObject(SettingsObject->GetFilterGroupSettings(FilterGroupIndex), true);

	return FReply::Handled();
}

/* HairCardGenWindow_Impl Definition
 *****************************************************************************/
static bool HairCardGenWindow_Impl::IsValidPath(const FDirectoryPath& ContentPath, const FString& PkgName)
{
	FString TargetFilename;
	bool bValidPath = FPackageName::TryConvertLongPackageNameToFilename(ContentPath.Path / PkgName, TargetFilename);

	if (bValidPath)
	{
		bValidPath = FPaths::ValidatePath(TargetFilename);
	}

	return bValidPath;
}

static bool HairCardGenWindow_Impl::DoFilesExist(const FDirectoryPath & ContentPath, const FString & PkgName)
{
	FString TargetFilename;
	if (!FPackageName::TryConvertLongPackageNameToFilename(ContentPath.Path / PkgName, TargetFilename))
	{
		return false;
	}

	TArray<FString> FoundFiles;
	// Assume that wild card matches are the associate texture files
	IFileManager::Get().FindFiles(FoundFiles, *(TargetFilename + TEXT("*.uasset")), /*Files =*/true, /*Directories =*/false);
	return FoundFiles.Num() > 0;
}

static bool HairCardGenWindow_Impl::PromptGroupSettingsDialog(TSharedPtr<SWindow> ParentWindow, TObjectPtr<UHairCardGeneratorPluginSettings> Settings, int FilterGroupIndex)
{
	// Compute centered window position based on max window size, which include when all categories are expanded
	const float WindowWidth = 450.0f;
	const float WindowHeight = 750.0f;
	FVector2D WindowSize = FVector2D(WindowWidth, WindowHeight); // Max window size it can get based on current slate

	FSlateRect WorkAreaRect = FSlateApplicationBase::Get().GetPreferredWorkArea();
	FVector2D DisplayTopLeft(WorkAreaRect.Left, WorkAreaRect.Top);
	FVector2D DisplaySize(WorkAreaRect.Right - WorkAreaRect.Left, WorkAreaRect.Bottom - WorkAreaRect.Top);

	float ScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(DisplayTopLeft.X, DisplayTopLeft.Y);
	WindowSize *= ScaleFactor;

	FVector2D WindowPosition = (DisplayTopLeft + (DisplaySize - WindowSize) / 2.0f) / ScaleFactor;

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("HairCardSettings.Group.Title", "Card Group Settings"))
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::None)
		.ClientSize(WindowSize)
		.ScreenPosition(WindowPosition);

	TSharedPtr<SHairCardGroupSettingsWindow> GroupSettingsOptionsWindow;
	Window->SetContent
	(
		SAssignNew(GroupSettingsOptionsWindow, SHairCardGroupSettingsWindow)
			.FilterGroupIndex(FilterGroupIndex)
			.SettingsObject(Settings)
			.WidgetWindow(Window)
			.MaxWindowHeight(WindowHeight)
			.MaxWindowWidth(WindowWidth)
	);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, /*bSlowTaskWindow =*/false);

	return true;
}

#undef LOCTEXT_NAMESPACE

