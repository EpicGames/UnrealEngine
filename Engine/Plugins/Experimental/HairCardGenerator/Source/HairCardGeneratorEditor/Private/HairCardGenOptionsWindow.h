// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroomAssetCards.h" // for UHairCardGenerationSettings
#include "HairCardGeneratorPluginSettings.h" // UHairCardGeneratorSettings

class UGroomAsset;
struct FHairGroupsCardsSourceDescription;

namespace HairCardGenWindow_Utils
{
	/**
	 * Opens a modal dialog, prompting the user to run hair card generation.
	 * 
	 * @param  OptionsObject	Settings structure defining the settings the user wants to run the generation with (may be modified on return)
	 * @raturn True if the user specified they want to run the generation, false if they canceled the process.
	 */
	bool PromptUserWithHairCardGenDialog(UHairCardGeneratorPluginSettings* OptionsObject);
}
