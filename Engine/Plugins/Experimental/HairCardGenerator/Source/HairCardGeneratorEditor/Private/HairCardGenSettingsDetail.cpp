// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardGenSettingsDetail.h"
#include "HairCardGeneratorPluginSettings.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Dom/JsonObject.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#include "HairCardGeneratorLog.h"

#define LOCTEXT_NAMESPACE "HairCardSettingsDetails"

void FHairCardSettingsDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
    TArray<TWeakObjectPtr<UHairCardGeneratorPluginSettings>> DetailObjects = DetailBuilder.GetObjectsOfTypeBeingCustomized<UHairCardGeneratorPluginSettings>();
    if ( DetailObjects.Num() != 1 || !DetailObjects[0].IsValid() )
    {
        SettingsPtr = nullptr;    
        return;
    }

    // Only support single object customized view (this is fine as HCS is a modal dialog anyway)
    SettingsPtr = DetailObjects[0];

    DetailBuilder.EditCategory(TEXT("Asset"));
    DetailBuilder.EditCategory(TEXT("Import"));

    IDetailCategoryBuilder& LodCategory = DetailBuilder.EditCategory(TEXT("Level Of Detail"));
    const TSharedPtr<IPropertyHandle> ReduceFromLODHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, bReduceCardsFromPreviousLOD));
    const TSharedPtr<IPropertyHandle> UseReservedTxHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, bUseReservedSpaceFromPreviousLOD));

    DetailBuilder.EditCategory(TEXT("Randomness"));
    IDetailCategoryBuilder& TxRenderCategory = DetailBuilder.EditCategory(TEXT("Texture Rendering"));

    // Specialized widget with dynamic tooltip for enabled/disabled derive from previous LOD checkbox
    DetailBuilder.HideProperty(ReduceFromLODHandle);
    LodCategory.AddCustomRow(LOCTEXT("ReduceFromLOD.Filter", "Reduce Cards from Previous LOD"))
    .NameContent()
    [
            SNew(STextBlock)
            .Text(LOCTEXT("ReduceFromLOD.Name", "Reduce Cards from Previous LOD"))
            .ToolTipText(ReduceFromLODHandle->GetToolTipText())
            .Font(IDetailLayoutBuilder::GetDetailFont())
    ]
    .ValueContent()
    [
        SNew(SCheckBox)
        .IsChecked(this, &FHairCardSettingsDetailCustomization::GetCheckValue, ReduceFromLODHandle)
        .OnCheckStateChanged(this, &FHairCardSettingsDetailCustomization::SetCheckValue, ReduceFromLODHandle)
        .IsEnabled(this, &FHairCardSettingsDetailCustomization::IsEnabledReduceFromLOD, ReduceFromLODHandle)
        .ToolTipText(this, &FHairCardSettingsDetailCustomization::ToolTipReduceFromLOD, ReduceFromLODHandle)
    ];

    // Specialized widget with dynamic tooltip for enabled/disabled use previous reserved texture space
    DetailBuilder.HideProperty(UseReservedTxHandle);
    TxRenderCategory.AddCustomRow(LOCTEXT("UseReservedTx.Filter", "Use Reserved Space from Previous LOD"))
    .NameContent()
    [
            SNew(STextBlock)
            .Text(LOCTEXT("UseReservedTx.Name", "Use Reserved Space from Previous LOD"))
            .ToolTipText(UseReservedTxHandle->GetToolTipText())
            .Font(IDetailLayoutBuilder::GetDetailFont())
    ]
    .ValueContent()
    [
        SNew(SCheckBox)
        .IsChecked(this, &FHairCardSettingsDetailCustomization::GetCheckValue, UseReservedTxHandle)
        .OnCheckStateChanged(this, &FHairCardSettingsDetailCustomization::SetCheckValue, UseReservedTxHandle)
        .IsEnabled(this, &FHairCardSettingsDetailCustomization::IsEnabledUseReservedTx, UseReservedTxHandle)
        .ToolTipText(this, &FHairCardSettingsDetailCustomization::ToolUseReservedTx, UseReservedTxHandle)
    ];
}


TSharedRef<IDetailCustomization> FHairCardSettingsDetailCustomization::MakeInstance()
{
    return MakeShareable(new FHairCardSettingsDetailCustomization());
}



ECheckBoxState FHairCardSettingsDetailCustomization::GetCheckValue(const TSharedPtr<IPropertyHandle> Property) const
{
    bool CheckVal;
    if ( Property.IsValid() && Property->GetValue(CheckVal) )
    {
        return (CheckVal) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
    }

    return ECheckBoxState::Undetermined;
}

void FHairCardSettingsDetailCustomization::SetCheckValue(ECheckBoxState NewState, TSharedPtr<IPropertyHandle> Property) const
{
    if ( Property.IsValid() )
    {
        Property->SetValue(NewState == ECheckBoxState::Checked);
    }
}

bool FHairCardSettingsDetailCustomization::IsEnabledReduceFromLOD(const TSharedPtr<IPropertyHandle> Property) const
{
    FText Ignore;
    return CheckReduceFromLOD(Property, Ignore);
}

FText FHairCardSettingsDetailCustomization::ToolTipReduceFromLOD(const TSharedPtr<IPropertyHandle> Property) const
{
    FText TooltipInfo;
    CheckReduceFromLOD(Property, TooltipInfo);

    return TooltipInfo;
}

bool FHairCardSettingsDetailCustomization::CheckReduceFromLOD(const TSharedPtr<IPropertyHandle> Property, FText& OutTooltipInfo) const
{
    OutTooltipInfo = Property->GetToolTipText();
    if ( !SettingsPtr.IsValid() )
    {
        return false;
    }

    return SettingsPtr.Get()->CanReduceFromLOD(&OutTooltipInfo);
}

bool FHairCardSettingsDetailCustomization::IsEnabledUseReservedTx(const TSharedPtr<IPropertyHandle> Property) const
{
    FText Ignore;
    return CheckUseReservedTx(Property, Ignore);
}


FText FHairCardSettingsDetailCustomization::ToolUseReservedTx(const TSharedPtr<IPropertyHandle> Property) const
{
    FText TooltipInfo;
    CheckUseReservedTx(Property, TooltipInfo);

    return TooltipInfo;
}

bool FHairCardSettingsDetailCustomization::CheckUseReservedTx(const TSharedPtr<IPropertyHandle> Property, FText& OutTooltipInfo) const
{
    OutTooltipInfo = Property->GetToolTipText();
    if ( !SettingsPtr.IsValid() )
    {
        return false;
    }

    return SettingsPtr.Get()->CanUseReservedTx(&OutTooltipInfo);
}


#undef LOCTEXT_NAMESPACE
