// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Styling/SlateTypes.h"

//Forward declaration of IPropertyHandle.
class IPropertyHandle;

// Forward declater hair card settings object
class UHairCardGeneratorPluginSettings;

// Detail panel customization for HairCardGeneratorPluginSettings
class FHairCardSettingsDetailCustomization : public IDetailCustomization
{
public:
    //Function that customizes the Details Panel.
    virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

    //Returns a static instance of the Details Panel customization.
    static TSharedRef<IDetailCustomization> MakeInstance();

private:
    ECheckBoxState GetCheckValue(const TSharedPtr<IPropertyHandle> Property) const;
    void SetCheckValue(ECheckBoxState NewState, const TSharedPtr<IPropertyHandle> Property) const;

    bool IsEnabledReduceFromLOD(const TSharedPtr<IPropertyHandle> Property) const;
    FText ToolTipReduceFromLOD(const TSharedPtr<IPropertyHandle> Property) const;
    bool CheckReduceFromLOD(const TSharedPtr<IPropertyHandle> Property, FText& OutTooltipInfo) const;

    bool IsEnabledUseReservedTx(const TSharedPtr<IPropertyHandle> Property) const;
    FText ToolUseReservedTx(const TSharedPtr<IPropertyHandle> Property) const;
    bool CheckUseReservedTx(const TSharedPtr<IPropertyHandle> Property, FText& OutTooltipInfo) const;

    TWeakObjectPtr<UHairCardGeneratorPluginSettings> SettingsPtr;
};
