// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardGenStrandFilter.h"

#include "HairCardGeneratorLog.h"


/* FHairStrandAttribsRefProxy Implementation
 *****************************************************************************/
bool FHairStrandAttribsRefProxy::IsValid() const
{
    UE_CLOGF(!StrandVertCounts.IsValid() || !VertexPositions.IsValid(), LogHairCardGenerator, Error, "Hair description has no vertex information!");
    UE_CLOGF(VertexPositions.GetNumElements() == 0, LogHairCardGenerator, Error, "Hair description has no vertices!")

    return StrandVertCounts.IsValid() && VertexPositions.IsValid() && (VertexPositions.GetNumElements() != 0);
}

bool FHairStrandAttribsRefProxy::IsGuideStrand() const
{
    return GuideStrands.IsValid() && (GuideStrands[FStrandID(StrandIndex)] != 0);
}

bool FHairStrandAttribsRefProxy::HasGroupID() const
{
    return StrandGroupIds.IsValid();
}

const int& FHairStrandAttribsRefProxy::GetGroupID() const
{
    return StrandGroupIds[FStrandID(StrandIndex)];
}

bool FHairStrandAttribsRefProxy::HasStrandWidth() const
{
    return StrandWidths.IsValid();
}

const float& FHairStrandAttribsRefProxy::GetStrandWidth() const
{
    return StrandWidths[FStrandID(StrandIndex)];
}

bool FHairStrandAttribsRefProxy::HasCardGroupName() const
{
    return StrandGroupCardsNames.IsValid();
}

const FName& FHairStrandAttribsRefProxy::GetCardGroupName() const
{
    return StrandGroupCardsNames[FStrandID(StrandIndex)];
}

bool FHairStrandAttribsRefProxy::HasBasisType() const
{
    return StrandBasisType.IsValid();
}

EGroomBasisType FHairStrandAttribsRefProxy::GetBasisType() const
{
    UEnum* BasisEnum = StaticEnum<EGroomBasisType>();
    return (StrandBasisType.IsValid() ? (EGroomBasisType)BasisEnum->GetValueByName(StrandBasisType[FStrandID(StrandIndex)]) : EGroomBasisType::NoBasis);
}

bool FHairStrandAttribsRefProxy::HasCurveType() const
{
    return StrandCurveType.IsValid();
}

EGroomCurveType FHairStrandAttribsRefProxy::GetCurveType() const
{
    UEnum* CurveEnum = StaticEnum<EGroomCurveType>();
    return (StrandCurveType.IsValid() ? (EGroomCurveType)CurveEnum->GetValueByName(StrandCurveType[FStrandID(StrandIndex)]) : EGroomCurveType::Linear);
}

bool FHairStrandAttribsRefProxy::HasKnots() const
{
    return StrandKnots.IsValid();
}

TArrayAttribute<const float> FHairStrandAttribsRefProxy::GetKnots() const
{
    return StrandKnots[FStrandID(StrandIndex)];
}

int FHairStrandAttribsRefProxy::NumVerts() const
{
    return StrandVertCounts[FStrandID(StrandIndex)];
}

const FVector3f& FHairStrandAttribsRefProxy::GetVertexPosition(int Index) const
{
    const FVertexID VertID(StartVertIndex + Index);
    return VertexPositions[VertID];
}

TArrayView<const FVector3f> FHairStrandAttribsRefProxy::GetVertexPositions() const
{
    const FStrandID StrandID(StrandIndex);
    int VertCount = StrandVertCounts[StrandID];

    return TArrayView<const FVector3f>(VertexPositions.GetArrayView(StartVertIndex).GetData(), VertCount);
}

bool FHairStrandAttribsRefProxy::HasVertexWidths() const
{
    return VertexWidths.IsValid();
}

const float& FHairStrandAttribsRefProxy::GetVertexWidth(int Index) const
{
    const FVertexID VertID(StartVertIndex + Index);
    return VertexWidths[VertID];
}

TArrayView<const float> FHairStrandAttribsRefProxy::GetVertexWidths() const
{
    const FStrandID StrandID(StrandIndex);
    int VertCount = StrandVertCounts[StrandID];

    return TArrayView<const float>(VertexWidths.GetArrayView(StartVertIndex).GetData(), VertCount);
}

bool FHairStrandAttribsRefProxy::HasVertexAO() const
{
	return VertexAOs.IsValid();
}

const float& FHairStrandAttribsRefProxy::GetVertexAO(int Index) const
{
	const FVertexID VertID(StartVertIndex + Index);
	return VertexAOs[VertID];
}

TArrayView<const float> FHairStrandAttribsRefProxy::GetVertexAOs() const
{
	const FStrandID StrandID(StrandIndex);
	int VertCount = StrandVertCounts[StrandID];

	return TArrayView<const float>(VertexAOs.GetArrayView(StartVertIndex).GetData(), VertCount);
}


FHairStrandAttribsRefProxy::FHairStrandAttribsRefProxy(const FHairDescription& InHairDescription)
    : StrandIndex(0)
    , StartVertIndex(0)
    , GuideStrands(InHairDescription.StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::Guide))
    , StrandGroupIds(InHairDescription.StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::GroupID))
    , StrandWidths(InHairDescription.StrandAttributes().GetAttributesRef<float>(HairAttribute::Strand::Width))
    , StrandGroupCardsNames(InHairDescription.StrandAttributes().GetAttributesRef<FName>(HairAttribute::Strand::GroupCardsName))
    , StrandBasisType(InHairDescription.StrandAttributes().GetAttributesRef<FName>(HairAttribute::Strand::BasisType))
    , StrandCurveType(InHairDescription.StrandAttributes().GetAttributesRef<FName>(HairAttribute::Strand::CurveType))
    , StrandKnots(InHairDescription.StrandAttributes().GetAttributesRef<TArrayAttribute<float>>(HairAttribute::Strand::Knots))
    , StrandVertCounts(InHairDescription.StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::VertexCount))
    , VertexPositions(InHairDescription.VertexAttributes().GetAttributesRef<FVector3f>(HairAttribute::Vertex::Position))
    , VertexWidths(InHairDescription.VertexAttributes().GetAttributesRef<float>(HairAttribute::Vertex::Width))
	, VertexAOs(InHairDescription.VertexAttributes().GetAttributesRef<float>(HairAttribute::Vertex::AO))
{}


void FHairStrandAttribsRefProxy::SetStrand(int InStrandIndex)
{
    int StartIdx = FMath::Min(InStrandIndex,StrandIndex);
    int EndIdx = FMath::Max(InStrandIndex,StrandIndex);

    // Accumulate vertex counts (for increment op will just add StrandVertCounts[StrandIndex])
    int dir = FMath::Sign(EndIdx-StartIdx);
    for ( int i=StartIdx; i < EndIdx; ++i )
        StartVertIndex += dir * StrandVertCounts[FStrandID(i)];

    StrandIndex = InStrandIndex;
}

/* FHairStrandAttribsIteratorEnd Implementation
 *****************************************************************************/
FHairStrandAttribsIteratorEnd::FHairStrandAttribsIteratorEnd(const FHairDescription& InHairDescription, int32 InEndIndex /* = INDEX_NONE */)
    : EndIndex((InEndIndex == INDEX_NONE) ? InHairDescription.GetNumStrands() : InEndIndex)
    , HairDescription(InHairDescription)
{}


/* FHairStrandAttribsConstIterator Implementation
 *****************************************************************************/
FHairStrandAttribsConstIterator::FHairStrandAttribsConstIterator(const FHairDescription& InHairDescription, int32 InStartIndex /* = 0 */, FHairStrandAttribsConstIterator::EIgnoreStrandFlags InIgnoreFlags)
    : StrandIndex(InStartIndex)
    , IgnoreFlags(InIgnoreFlags)
    , HairDescription(InHairDescription)
    , AttribsProxy(HairDescription)
{
    if ( StrandIndex < HairDescription.GetNumStrands() )
        FindFiltered();
}

const FHairStrandAttribsRefProxy& FHairStrandAttribsConstIterator::operator*() const
{
    return AttribsProxy;
}

const FHairStrandAttribsRefProxy* FHairStrandAttribsConstIterator::operator->() const
{
    return &AttribsProxy;
}

FHairStrandAttribsConstIterator& FHairStrandAttribsConstIterator::operator++()
{
    if ( StrandIndex >= HairDescription.GetNumStrands() )
    {
        return *this;
    }

    StrandIndex += 1;
    FindFiltered();

    return *this;
}

bool FHairStrandAttribsConstIterator::operator!=(const FHairStrandAttribsConstIterator& Iter) const
{
    UE_CLOGF((&Iter.HairDescription != &HairDescription), LogHairCardGenerator, Error, "Invalid strand iterator comparison (different hair descriptions)!");
    return (StrandIndex != Iter.StrandIndex);
}

bool FHairStrandAttribsConstIterator::operator!=(const FHairStrandAttribsIteratorEnd& Iter) const
{
    return (StrandIndex < Iter.EndIndex);
}

void FHairStrandAttribsConstIterator::FindFiltered()
{
    for ( ; StrandIndex < HairDescription.GetNumStrands(); ++StrandIndex )
    {
        AttribsProxy.SetStrand(StrandIndex);
        if ( (IgnoreFlags & IgnoreGuides) && AttribsProxy.IsGuideStrand() )
            continue;

        if ( (IgnoreFlags & IgnoreEmpty) && (AttribsProxy.NumVerts() == 0) )
            continue;

        return;
    }
}


/* FStrandAttributesRange Implementation
 *****************************************************************************/

FStrandAttributesRange::FStrandAttributesRange(const FHairDescription& HairDesc, FHairStrandAttribsConstIterator::EIgnoreStrandFlags InIgnoreFlags/* = FHairStrandAttribsConstIterator::DefaultIgnore*/)
    : HairDescription(HairDesc)
    , BeginIter(HairDesc, 0, InIgnoreFlags)
{}

bool FStrandAttributesRange::IsValid() const
{
    return BeginIter->IsValid();
}

int FStrandAttributesRange::NumStrands() const
{
    return HairDescription.GetNumStrands();
}

int FStrandAttributesRange::NumVerts() const
{
    return BeginIter->IsValid() ? BeginIter->VertexPositions.GetNumElements() : 0;
}

const FHairStrandAttribsRefProxy& FStrandAttributesRange::First() const
{
    return *BeginIter;
}

FHairStrandAttribsConstIterator FStrandAttributesRange::begin() const
{
    return BeginIter;
}

FHairStrandAttribsIteratorEnd FStrandAttributesRange::end() const
{
    return FHairStrandAttribsIteratorEnd(HairDescription);
}

/* FStrandFilterOp_And Implementation
 *****************************************************************************/
FStrandFilterOp_And::FStrandFilterOp_And(TUniquePtr<FComposableStrandFilterOp> InLeftOp, TUniquePtr<FComposableStrandFilterOp> InRightOp)
    : LeftOp(MoveTemp(InLeftOp)), RightOp(MoveTemp(InRightOp))
{}

bool FStrandFilterOp_And::FilterStrand(const FHairStrandAttribsRefProxy& StrandAttributes)
{
    return (LeftOp->FilterStrand(StrandAttributes) && RightOp->FilterStrand(StrandAttributes));
}

/* FStrandFilterOp_Or Implementation
 *****************************************************************************/
FStrandFilterOp_Or::FStrandFilterOp_Or(TUniquePtr<FComposableStrandFilterOp> InLeftOp, TUniquePtr<FComposableStrandFilterOp> InRightOp)
    : LeftOp(MoveTemp(InLeftOp)), RightOp(MoveTemp(InRightOp))
{}

bool FStrandFilterOp_Or::FilterStrand(const FHairStrandAttribsRefProxy& StrandAttributes)
{
    return (LeftOp->FilterStrand(StrandAttributes) || RightOp->FilterStrand(StrandAttributes));
}

/* FStrandFilterOp_InCardGroup Implementation
 *****************************************************************************/
FStrandFilterOp_InCardGroup::FStrandFilterOp_InCardGroup(const TSet<FName>& InCardGroupIds)
    : CardGroupNames(InCardGroupIds)
{}

bool FStrandFilterOp_InCardGroup::FilterStrand(const FHairStrandAttribsRefProxy& StrandAttributes)
{
    // Return true for all strands if no valid card group id, otherwise check if group is in set
    return StrandAttributes.HasCardGroupName() ? CardGroupNames.Contains(StrandAttributes.GetCardGroupName()) : true;
}
