// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "GroomCacheData.h"

#include "Misc/Optional.h"


// Reference structure used to proxy a particular strand's attributes
class FStrandAttributesRange;
class FHairStrandAttribsConstIterator;
class FHairStrandAttribsRefProxy
{
    friend class FStrandAttributesRange;
    friend class FHairStrandAttribsConstIterator;
public:
    bool IsValid() const;

    bool IsGuideStrand() const;

    bool HasGroupID() const;
    const int& GetGroupID() const;

    bool HasStrandWidth() const;
    const float& GetStrandWidth() const;

    bool HasCardGroupName() const;
    const FName& GetCardGroupName() const;

    bool HasBasisType() const;
    EGroomBasisType GetBasisType() const;

    bool HasCurveType() const;
    EGroomCurveType GetCurveType() const;

    bool HasKnots() const;
    TArrayAttribute<const float> GetKnots() const;

    // Per-strand vertex info
    int NumVerts() const;
    const FVector3f& GetVertexPosition(int Index) const;
    TArrayView<const FVector3f> GetVertexPositions() const;

    bool HasVertexWidths() const;
    const float& GetVertexWidth(int Index) const;
    TArrayView<const float> GetVertexWidths() const;

	bool HasVertexAO() const;
	const float& GetVertexAO(int Index) const;
	TArrayView<const float> GetVertexAOs() const;


private:
    FHairStrandAttribsRefProxy() = delete;
    FHairStrandAttribsRefProxy(const FHairDescription& InHairDescription);

    void SetStrand(int InStrandIndex);

    int StrandIndex;
    int StartVertIndex;

    TStrandAttributesConstRef<int> GuideStrands;
    TStrandAttributesConstRef<int> StrandGroupIds;
    TStrandAttributesConstRef<float> StrandWidths;
    TStrandAttributesConstRef<FName> StrandGroupCardsNames;

    TStrandAttributesConstRef<FName> StrandBasisType;
    TStrandAttributesConstRef<FName> StrandCurveType;
    TStrandAttributesConstRef<TArrayAttribute<float>> StrandKnots;

    TStrandAttributesConstRef<int> StrandVertCounts;
    TVertexAttributesConstRef<FVector3f> VertexPositions;
    TVertexAttributesConstRef<float> VertexWidths;
	TVertexAttributesConstRef<float> VertexAOs;
};


// Helper iterator clases to make it easier to iterate through strands and get access to per-strand attributes
class FHairStrandAttribsIteratorEnd
{
    friend class FHairStrandAttribsConstIterator;
public:
    FHairStrandAttribsIteratorEnd(const FHairDescription& InHairDescription, int32 InEndIndex = INDEX_NONE);

private:
    int32 EndIndex;
    const FHairDescription& HairDescription;
};

class FHairStrandAttribsConstIterator
{
public:
    enum EIgnoreStrandFlags
    {
        // Don't ignore anything
        IgnoreNone = 0x0,

        // Ignore guide strands in traversal
        IgnoreGuides = 0x01,
        // Ignore empty (NumVerts==0) strands in traversal
        IgnoreEmpty = 0x02,

        // Default setting is to ignore guide and empty strands
        DefaultIgnore = IgnoreGuides | IgnoreEmpty,
    };

public:
    FHairStrandAttribsConstIterator(const FHairDescription& InHairDescription, int32 InStartIndex = 0, EIgnoreStrandFlags InIgnoreFlags = EIgnoreStrandFlags::DefaultIgnore);

    const FHairStrandAttribsRefProxy& operator*() const;
    const FHairStrandAttribsRefProxy* operator->() const;
    FHairStrandAttribsConstIterator& operator++();
    bool operator!=(const FHairStrandAttribsConstIterator& Iter) const;
    bool operator!=(const FHairStrandAttribsIteratorEnd& Iter) const;

private:
    FHairStrandAttribsConstIterator() = delete;

    void FindFiltered();

    int32 StrandIndex;
    EIgnoreStrandFlags IgnoreFlags;
    const FHairDescription& HairDescription;

    FHairStrandAttribsRefProxy AttribsProxy;
};

// Helper traversal class allows use of range-for syntax for getting per-strand attributes
class FStrandAttributesRange
{
public:
    /// Creates a helper class which will return iterator over each strand in a hair description with an attribute proxy to make it relatively easy to access per-strand attributes
    /// @param HairDesc FHairDescription for the groom
    /// @param InIgnoreFlags Flags indicating what strands to ignore during traversal (defaults to ignore guide strands and empty (vert) strands)
	FStrandAttributesRange(const FHairDescription& HairDesc, FHairStrandAttribsConstIterator::EIgnoreStrandFlags InIgnoreFlags = FHairStrandAttribsConstIterator::DefaultIgnore);

    bool IsValid() const;
    int NumStrands() const;
    int NumVerts() const;

    const FHairStrandAttribsRefProxy& First() const;

    FHairStrandAttribsConstIterator begin() const;
    FHairStrandAttribsIteratorEnd end() const;
private:
    const FHairDescription& HairDescription;
    const FHairStrandAttribsConstIterator BeginIter;
};


// Base class for all strand filters
class FComposableStrandFilterOp
{
public:
	virtual bool FilterStrand(const FHairStrandAttribsRefProxy& StrandAttributes) = 0;

    virtual ~FComposableStrandFilterOp() = default;
protected:
	FComposableStrandFilterOp() = default;
};

class FStrandFilterOp_And : public FComposableStrandFilterOp
{
public:
    FStrandFilterOp_And(TUniquePtr<FComposableStrandFilterOp> InLeftOp, TUniquePtr<FComposableStrandFilterOp> InRightOp);
    virtual bool FilterStrand(const FHairStrandAttribsRefProxy& StrandAttributes) override;
    
private:
    TUniquePtr<FComposableStrandFilterOp> LeftOp;
    TUniquePtr<FComposableStrandFilterOp> RightOp;
};

class FStrandFilterOp_Or : public FComposableStrandFilterOp
{
public:
    FStrandFilterOp_Or(TUniquePtr<FComposableStrandFilterOp> InLeftOp, TUniquePtr<FComposableStrandFilterOp> InRightOp);
    virtual bool FilterStrand(const FHairStrandAttribsRefProxy& StrandAttributes) override;

private:
    TUniquePtr<FComposableStrandFilterOp> LeftOp;
    TUniquePtr<FComposableStrandFilterOp> RightOp;
};

class FStrandFilterOp_InCardGroup : public FComposableStrandFilterOp
{
public:
	FStrandFilterOp_InCardGroup(const TSet<FName>& InCardGroupIds);
    virtual bool FilterStrand(const FHairStrandAttribsRefProxy& StrandAttributes) override;

private:
	TSet<FName> CardGroupNames;
};
