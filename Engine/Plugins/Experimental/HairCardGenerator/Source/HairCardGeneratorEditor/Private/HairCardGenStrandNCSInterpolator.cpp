// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardGenStrandNCSInterpolator.h"
#include <iostream>

FHairCardGenStrandNSCInterpolator::FHairCardGenStrandNSCInterpolator(const TArray<FVector>& Positions, const TArray<float>& Widths, const TArray<float>& AOs)
{
    int NumPoints = Positions.Num();

	TArray<float> X, Y, Z;
	X.SetNum(NumPoints);
	Y.SetNum(NumPoints);
	Z.SetNum(NumPoints);

	for (int i = 0; i < NumPoints; i++) {
		X[i] = Positions[i].X;
        Y[i] = Positions[i].Y;
        Z[i] = Positions[i].Z;
	}

    TArray<float> DistancesFromStart;
	DistancesFromStart.SetNum(NumPoints);
    DistancesFromStart[0] = 0;
	for (int i = 1; i < NumPoints; ++i)
	{
		DistancesFromStart[i] = DistancesFromStart[i - 1] + FVector::Dist(Positions[i], Positions[i - 1]);
	}
	for (int i = 1; i < NumPoints; ++i)
	{
		DistancesFromStart[i] = (DistancesFromStart[i] + float(i) * 0.1 * DistancesFromStart.Last()) / ((0.1 * float(NumPoints) + 0.9) * DistancesFromStart.Last());
	}

	Splines.Reserve(5);

	Splines.Add(FHairCardGenNaturalCubicSplines(DistancesFromStart, X));
	Splines.Add(FHairCardGenNaturalCubicSplines(DistancesFromStart, Y));
	Splines.Add(FHairCardGenNaturalCubicSplines(DistancesFromStart, Z));
	Splines.Add(FHairCardGenNaturalCubicSplines(DistancesFromStart, Widths));
	if (AOs.Num() > 0)
	{
		Splines.Add(FHairCardGenNaturalCubicSplines(DistancesFromStart, AOs));
	}
}

FHairCardGenStrandNSCInterpolator::StrandInterpolationResult FHairCardGenStrandNSCInterpolator::GetInterpolatedStrand(const int NumInterpolatedPoints)
{
	StrandInterpolationResult Result;

	Result.Positions.SetNum(NumInterpolatedPoints * 3);
	Result.Widths.SetNum(NumInterpolatedPoints);
	if (Splines.Num() == 5)
	{
		Result.AOs.SetNum(NumInterpolatedPoints);
	}

	for (int i = 0; i < NumInterpolatedPoints; i++)
	{
		float T = float(i) / float(NumInterpolatedPoints - 1);

		for (int j = 0; j < 3; j++)
		{
			Result.Positions[i * 3 + j] = Splines[j](T);
		}
		Result.Widths[i] = Splines[3](T);

		if (Splines.Num() == 5)
		{
			Result.AOs[i] = Splines[4](T);
		}
	}

	return Result;
}