// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/Vector.h"
#include "HairCardGenNaturalCubicSplines.h"

class FHairCardGenStrandNSCInterpolator
{
private:
    TArray<FHairCardGenNaturalCubicSplines> Splines;

public:
    struct StrandInterpolationResult
    {
        TArray<float> Positions;
        TArray<float> Widths;
		TArray<float> AOs;
    };

public:
	FHairCardGenStrandNSCInterpolator(const TArray<FVector>& Positions, const TArray<float>& Widths, const TArray<float>& AOs);
    StrandInterpolationResult GetInterpolatedStrand(const int NumInterpolatedPoints);
}; 
