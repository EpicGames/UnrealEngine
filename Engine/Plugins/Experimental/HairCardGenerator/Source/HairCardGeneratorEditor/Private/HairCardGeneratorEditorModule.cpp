// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardGeneratorEditorModule.h"
#include "IHairCardGeneratorEditor.h"
#include "Engine/Texture2D.h"
#include "Factories/TextureFactory.h"
#include "HairCardGeneratorLog.h"
#include "HairCardGenOptionsWindow.h"
#include "HairCardGeneratorPluginSettings.h"
#include "HairCardGenControllerBase.h"
#include "HairCardGenTypes.h"
#include "HairCardGeneratorEditorSettings.h"
#include "HairCardGenCardSubdivider.h"
#include "HairCardGenStrandNCSInterpolator.h"
#include "HairCardGenStrandFilter.h"
#include "HairCardGenSettingsDetail.h"

#include "AssetToolsModule.h"
#include "PropertyEditorModule.h"
#include "AutomatedAssetImportData.h"
#include "GroomAsset.h"
#include "GroomAssetCards.h" // for GetHairTextureLayoutTextureCount
#include "GroomCacheData.h" // for EGroomBasisType, EGroomCurveType
#include "HairDescription.h"
#include "ObjectTools.h"
#include "AssetRegistry/AssetRegistryModule.h" // for GetAssetByObjectPath()
#include "Engine/StaticMesh.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/UObjectHash.h" // for GetObjectsOfClass()
#include "UObject/WeakObjectPtrTemplates.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

DEFINE_LOG_CATEGORY(LogHairCardGenerator)
#define LOCTEXT_NAMESPACE "HairCardGeneratorEd"

LLM_DEFINE_TAG(HairCardGenerator);

/* Used for keeping track of the editor settings for importing generated data
*/
struct FHairCardImportSettings
{
	FString MeshName;
	TArray<FString> TextureNames;
};

/* FHairCardGeneratorHelper Declaration
 *****************************************************************************/

// Singleton class for caching a pointer to the hair card generator
class FHairCardGenController
{
public:
	static TObjectPtr<UHairCardGenControllerBase> GetHairCardGenerator();

private:
	static UGroomAsset* GroomAsset;
	static TWeakObjectPtr<UHairCardGenControllerBase> HairCardGenController;
};

TWeakObjectPtr<UHairCardGenControllerBase> FHairCardGenController::HairCardGenController = nullptr;

TObjectPtr<UHairCardGenControllerBase> FHairCardGenController::GetHairCardGenerator()
{
	if ( HairCardGenController.IsValid() )
		return HairCardGenController.Get();

	// Find the HairCardGenController in Python (or wherever it may be implemented -- could be moved to C++/Blueprints/etc.)
	TArray<UObject*> HairCardGenControllers;
	GetObjectsOfClass(UHairCardGenControllerBase::StaticClass(), HairCardGenControllers);

	if ( HairCardGenControllers.Num() == 0 )
	{
		UE_LOGF(LogHairCardGenerator, Error, "Failed to find an active hair card controller.");
		return nullptr;
	}

	if ( HairCardGenControllers.Num() != 1 )
		UE_LOGF(LogHairCardGenerator, Warning, "More than one active hair-card controller (using first controller found).")

	HairCardGenController = TWeakObjectPtr<UHairCardGenControllerBase>(static_cast<UHairCardGenControllerBase*>(HairCardGenControllers[0]));
	return HairCardGenController.Get();
}

/* HairCardGeneratorEditor_Impl Declaration
 *****************************************************************************/

// Static namespace with helper functions for commands/generation tasks
namespace HairCardGeneratorEditor_Impl
{
	static TObjectPtr<UGroomAsset> ParseGroomAssetArg(const TArray<FString>& Args, bool bOptional);
	static FHairGroupsCardsSourceDescription* FindCardSourceDescription(TObjectPtr<UGroomAsset> Groom, int32 LODIndex);
	static TObjectPtr<UHairCardGeneratorPluginSettings> CreateGroomSettingsLast(TObjectPtr<UGroomAsset> Groom, const FHairGroupsCardsSourceDescription& SourceDesc, 
		const FString& DefaultDirectory = "", const FString& DefaultName = "");

	static void DbgCmdCacheGroom(const TArray<FString>& Args);

	static bool LoadGroomData(UGroomAsset* GroomAsset, const bool SaveCached = false);
	static bool BuildHairCardGenGroomData(const int PointsPerStrand, const FHairDescription& InGroomData, FHairCardGen_GroomData& Out);
	static bool GenerateCardsForCardGroup(const TObjectPtr<const UHairCardGeneratorPluginSettings> GenerationSettings, const int32 FilterIndex, const uint8 GenFlags);
	static bool GenerateCardsGeometry(TObjectPtr<const UHairCardGeneratorPluginSettings> Settings, int index, FHairCardGeomData& OutGeomData, FHairCardMeshData& OutMeshData, int32& OutCardCount);
	static void GetTextureImportNames(UHairCardGeneratorPluginSettings* Settings, FHairCardImportSettings& ImportSettings);
	static TArray<UObject*> ImportHairCardTextures(const FString& ImportSrc, const FString& ImportDst, const FHairCardImportSettings& ImportSettings);
	static UStaticMesh* CreateMeshInPackage(const FString& DestinationPackage, const FString& BaseFilename, int GroupID);
	static int32 AssignImportedTextures(const TArray<UObject*>& ImportedAssets, const FHairCardImportSettings& ImportSettings, FHairGroupsCardsSourceDescription& Assignee);
	static void CreateFinishNotification(bool bSuccess, const FText& NotificationText);

	static TWeakObjectPtr<UGroomAsset> LastGroom = nullptr;
}


static FAutoConsoleCommand CCmdLoadGroom(
	TEXT("HairCardSolver.CacheGroom"),
	TEXT(""),
	FConsoleCommandWithArgsDelegate::CreateStatic(HairCardGeneratorEditor_Impl::DbgCmdCacheGroom));

/* FHairCardGeneratorUtils Definition
 *****************************************************************************/

bool FHairCardGeneratorUtils::BuildGenerationSettings(const bool bQuerySettings, UGroomAsset* NewGroomAsset, FHairGroupsCardsSourceDescription& CardsDesc,
	TObjectPtr<UHairCardGeneratorPluginSettings>& GenerationSettings, uint8& AllGenFlags, uint8& PipelineGenFlags, const FCardsGenerationAdvancedOptions& AdvancedOptions)
{
	// Find the HairCardGenController in Python (or wherever it may be implemented -- could be moved to C++/Blueprints/etc.)
	TObjectPtr<UHairCardGenControllerBase> CardGenController = FHairCardGenController::GetHairCardGenerator();
	if ( !CardGenController )
	{
		UE_LOGF(LogHairCardGenerator, Error, "No hair card generator controller found, cannot generate hair cards!");
		return false;
	}

	UE_CLOGF(CardsDesc.GenerationSettings && !UHairCardGeneratorPluginSettings::IsCompatibleSettings(CardsDesc.GenerationSettings), LogHairCardGenerator, Warning, "Old hair-card generation settings (from a different generator) will be discarded when you run this generator.");

	GenerationSettings = CardGenController->GetGroomSettings(NewGroomAsset, CardsDesc.LODIndex);
	if ( !GenerationSettings )
	{
		GenerationSettings = HairCardGeneratorEditor_Impl::CreateGroomSettingsLast(NewGroomAsset, CardsDesc, AdvancedOptions.DefaultDirectory, AdvancedOptions.DefaultName);
	}
	else
	{
		// Need to update all non-generation UI settings in case they've changed on groom asset
		GenerationSettings->SetGenerateForGroomGroup(CardsDesc.GroupIndex);
		GenerationSettings->PostResetUpdates();
	}

	if(bQuerySettings)
	{
		if ( !HairCardGenWindow_Utils::PromptUserWithHairCardGenDialog(GenerationSettings) )
		{
			return false;
		}
	}
	else
	{
		GenerationSettings->bGenerateGeometryForAllGroups = AdvancedOptions.bGenerateGeometryForAllGroups;
		GenerationSettings->bReduceCardsFromPreviousLOD = AdvancedOptions.bReduceCardsFromPreviousLOD;
		GenerationSettings->RandomSeed = AdvancedOptions.RandomSeed;
		GenerationSettings->bUseReservedSpaceFromPreviousLOD = AdvancedOptions.bUseReservedSpaceFromPreviousLOD;
		GenerationSettings->AtlasSize = static_cast<EHairCardAtlasSize>(AdvancedOptions.AtlasSize);
		GenerationSettings->ReserveTextureSpaceLOD = AdvancedOptions.ReserveTextureSpaceLOD;
		GenerationSettings->bUseGroomAssetStrandWidth = AdvancedOptions.bUseGroomAssetStrandWidth;
	}

	// Check primary settings object merged and all group settings for differences
	AllGenFlags = GenerationSettings->GetAllPipelineGeneratedDifferences();
	if ( !GenerationSettings->CheckGenerationFlags(AllGenFlags, EHairCardGenerationPipeline::All) && CardsDesc.ImportedMesh != nullptr )
	{
		UE_LOGF(LogHairCardGenerator, Display, "No settings changes, skipping hair card run");
		HairCardGeneratorEditor_Impl::CreateFinishNotification(true, LOCTEXT("HairCardGen.UpToDate", "Hair card assets are up to date"));
		// FIXME: Currently returning true clears textures unless a reimport is run
		return false;
	}

	// Force reimport if we are going to run any pipeline step
	AllGenFlags |= (uint8)EHairCardGenerationPipeline::ImportUpdate;

	// Check primary settings object (without group settings) used to force reruns for e.g. random seed changes
	PipelineGenFlags = GenerationSettings->GetPipelineGeneratedDifferences();

	// Write the settings that we will attempt to output to card desc (won't be stored on failure see FGroomRenderingDetails::OnGenerateCardDataUsingPlugin for details)
	CardsDesc.GenerationSettings = GenerationSettings;

	// Need to set the CardsDesc texture layout appropriately in case we succeed
	CardsDesc.Textures.SetLayout(GenerationSettings->ChannelLayout);

	// TODO: On error will force full regen (is this the best we can do?)
	GenerationSettings->ClearPipelineGenerated();

	// Output generation settings to json file in Metadata folder
	GenerationSettings->WriteGenerationSettings();
	
	return true;
}


bool FHairCardGeneratorUtils::LoadGenerationSettings(const TObjectPtr<UHairCardGeneratorPluginSettings>& GenerationSettings)
{
	// Find the HairCardGenController in Python (or wherever it may be implemented -- could be moved to C++/Blueprints/etc.)
	TObjectPtr<UHairCardGenControllerBase> CardGenController = FHairCardGenController::GetHairCardGenerator();
	if ( !CardGenController )
	{
		UE_LOGF(LogHairCardGenerator, Error, "No hair card generator controller found, cannot generate hair cards!");
		return false;
	}
	
	if ( !CardGenController->LoadSettings(GenerationSettings) )
	{
		UE_LOGF(LogHairCardGenerator, Error, "Failed to load hair card generation settings. See log for details.");
		HairCardGeneratorEditor_Impl::CreateFinishNotification(false, LOCTEXT("HairCardGen.FailedSettingsLoad", "Failed to load hair card generation settings. See log for details."));
		return false;
	}
	return true;
}

bool FHairCardGeneratorUtils::GenerateCardsClumps(const TObjectPtr<const UHairCardGeneratorPluginSettings>& GenerationSettings, const int32 FilterIndex, const uint8 GenFlags, FHairCardClumpData& OutClumpData)
{
	TObjectPtr<UHairCardGenControllerBase> CardGenController = FHairCardGenController::GetHairCardGenerator();
	if ( !CardGenController )
	{
		return false;
	}

	if ( UHairCardGeneratorPluginSettings::CheckGenerationFlags(GenFlags, EHairCardGenerationPipeline::StrandClustering) && !GenerationSettings->bReduceCardsFromPreviousLOD )
	{
		if (!CardGenController->GenerateClumps(FilterIndex, OutClumpData))
		{
			UE_LOGF(LogHairCardGenerator, Error, "Strand clustering failed for settings group %d. Check the log for details.", FilterIndex);
			return false;
		}
		if (!CardGenController->SaveClumpData(FilterIndex, OutClumpData))
		{
			UE_LOG(LogHairCardGenerator, Warning, TEXT("Failed to save clump data for settings group %d."), FilterIndex);
		}
	}
	return true;
}

bool FHairCardGeneratorUtils::GenerateCardsGeometry(const TObjectPtr<const UHairCardGeneratorPluginSettings>& GenerationSettings, const int32 FilterIndex, const uint8 GenFlags, FHairCardGeomData& OutGeomData, FHairCardMeshData& OutMeshData, int32& OutCardCount)
{
	TObjectPtr<UHairCardGenControllerBase> CardGenController = FHairCardGenController::GetHairCardGenerator();
	if ( !CardGenController )
	{
		return false;
	}
	if ( UHairCardGeneratorPluginSettings::CheckGenerationFlags(GenFlags, EHairCardGenerationPipeline::GeometryGeneration) )
	{
		if ( !HairCardGeneratorEditor_Impl::GenerateCardsGeometry(GenerationSettings, FilterIndex, OutGeomData, OutMeshData, OutCardCount) )
		{
			UE_LOGF(LogHairCardGenerator, Error, "Cards geometry failed for settings group %d. Check the log for details.", FilterIndex)
			return false;
		}
		if (!CardGenController->SaveCardData(FilterIndex, OutGeomData, OutMeshData, OutCardCount))
		{
			UE_LOG(LogHairCardGenerator, Warning, TEXT("Failed to save card data for settings group %d."), FilterIndex);
		}
	}
	return true;
}

bool FHairCardGeneratorUtils::GenerateCardsTexturesClusters(const TObjectPtr<const UHairCardGeneratorPluginSettings>& GenerationSettings, const int32 FilterIndex, const uint8 GenFlags,
	FHairCardTextureClusterData& OutClusterData)
{
	TObjectPtr<UHairCardGenControllerBase> CardGenController = FHairCardGenController::GetHairCardGenerator();
	if ( !CardGenController )
	{
		return false;
	}
	if ( UHairCardGeneratorPluginSettings::CheckGenerationFlags(GenFlags, EHairCardGenerationPipeline::TextureClustering) && !GenerationSettings->bReduceCardsFromPreviousLOD )
	{
		FHairCardGeomData GeomData;
		FHairCardMeshData MeshData;
		int32 CardCount = 0;
		if (!CardGenController->LoadCardData(FilterIndex, GeomData, MeshData, CardCount))
		{
			UE_LOG(LogHairCardGenerator, Error, TEXT("Failed to load card data for settings group %d."), FilterIndex);
			return false;
		}

		if (!CardGenController->ClusterTextures(FilterIndex, GeomData, MeshData, OutClusterData))
		{
			UE_LOGF(LogHairCardGenerator, Error, "Texture clustering failed for settings group %d. Check the log for details.", FilterIndex)
			return false;
		}
		if (!CardGenController->SaveTextureClusterData(FilterIndex, OutClusterData))
		{
			UE_LOG(LogHairCardGenerator, Warning, TEXT("Failed to save texture cluster data for settings group %d."), FilterIndex);
		}
	}
	return true;
}

bool FHairCardGeneratorUtils::GenerateTexturesLayoutAndAtlases(const TObjectPtr<const UHairCardGeneratorPluginSettings>& GenerationSettings, const uint8 GenFlags, FHairCardAtlasLayoutData& OutLayoutData, FHairCardAtlasUVData& OutAtlasUVData)
{
	TObjectPtr<UHairCardGenControllerBase> CardGenController = FHairCardGenController::GetHairCardGenerator();
	if ( !CardGenController )
	{
		return false;
	}

	const int32 NumGroups = GenerationSettings->GetFilterGroupSettings().Num();
	TArray<FHairCardGeomData> AllGeom;
	TArray<FHairCardMeshData> AllMesh;
	TArray<FHairCardTextureClusterData> AllCluster;
	TArray<int32> AllCardCount;
	AllGeom.SetNum(NumGroups);
	AllMesh.SetNum(NumGroups);
	AllCluster.SetNum(NumGroups);
	AllCardCount.SetNum(NumGroups);
	for (int32 i = 0; i < NumGroups; ++i)
	{
		if (!CardGenController->LoadCardData(i, AllGeom[i], AllMesh[i], AllCardCount[i]))
		{
			UE_LOG(LogHairCardGenerator, Error, TEXT("Failed to load card data for group %d in texture layout step."), i);
			return false;
		}
		if (!CardGenController->LoadTextureClusterData(i, AllCluster[i]))
		{
			UE_LOG(LogHairCardGenerator, Error, TEXT("Failed to load texture cluster data for group %d in texture layout step."), i);
			return false;
		}
	}

	// Layout is fast (bin-packing only) — always run it so OutLayoutData and OutAtlasUVData are
	// reliably populated for callers (e.g. the Dataflow node repopulates VertexTextureUVs each eval).
	// Skip only when deriving geometry from a previous LOD, where the parent layout is reused.
	const bool bRunLayout    = !GenerationSettings->bReduceCardsFromPreviousLOD;
	const bool bRunRendering = UHairCardGeneratorPluginSettings::CheckGenerationFlags(GenFlags, EHairCardGenerationPipeline::TextureRendering) && !GenerationSettings->bReduceCardsFromPreviousLOD;

	if (bRunLayout)
	{
		FHairCardAtlasBinpackData BinpackData;
		if (!CardGenController->GenerateTextureLayout(AllGeom, AllMesh, AllCluster, OutLayoutData, BinpackData, OutAtlasUVData))
		{
			UE_LOGF(LogHairCardGenerator, Error, "Texture layout failed. See log for details.");
			return false;
		}
		CardGenController->SaveAtlasLayoutData(OutLayoutData);
		CardGenController->SaveAtlasBinpackData(BinpackData);
	}

	if (bRunRendering)
	{
		FHairCardAtlasBinpackData BinpackData;
		if (!CardGenController->LoadAtlasBinpackData(BinpackData))
		{
			UE_LOG(LogHairCardGenerator, Error, TEXT("Failed to load atlas binpack data for texture rendering step."));
			return false;
		}
		if (!CardGenController->GenerateTextureAtlases(AllGeom, AllMesh, AllCluster, OutLayoutData, BinpackData, -1.0f))
		{
			UE_LOGF(LogHairCardGenerator, Error, "Texture atlas creation failed. See log for details.");
			return false;
		}
	}

	return true;
}

bool FHairCardGeneratorUtils::RunCardsGeneration(const TObjectPtr<UHairCardGeneratorPluginSettings>& GenerationSettings, const uint8 PipelineFlags, const FGeneratorFunction& PipelineFunction, const bool bCheckFlags)
{
	// Run card group generation in loop
	bool bAllGenerated = true;
	for ( int32 FilterIndex = 0; FilterIndex < GenerationSettings->GetFilterGroupSettings().Num(); ++FilterIndex )
	{
		const uint8 GroupGenFlags = bCheckFlags ? (PipelineFlags | GenerationSettings->GetPipelineFilterGroupDifferences(FilterIndex)) : static_cast<uint8>(EHairCardGenerationPipeline::All);

		// Skip if no group update flags required for current group
		if ( !GenerationSettings->CheckGenerationFlags(GroupGenFlags, EHairCardGenerationPipeline::GroupUpdate) )
		{
			UE_LOGF(LogHairCardGenerator, Display, "Setttings group %d already generated. Skipping.", FilterIndex);
			continue;
		}

		// TODO: On errors this will force full regeneration (because we clear the group gen object)
		GenerationSettings->ClearPipelineGeneratedFilterGroup(FilterIndex);
		bAllGenerated &= PipelineFunction(GenerationSettings, FilterIndex, GroupGenFlags);

		// TODO: It might be useful to handle user cancellation by breaking vs. errors by continuing here
		if ( !bAllGenerated )
		{
			break;
		}

		GenerationSettings->WritePipelineGeneratedFilterGroup(FilterIndex);
	}

	if ( !bAllGenerated )
	{
		UE_LOGF(LogHairCardGenerator, Error, "Failed to generate cards for all groups. See log for details.");
		HairCardGeneratorEditor_Impl::CreateFinishNotification(false, LOCTEXT("HairCardGen.FailedGeneration", "Failed to generate cards for all groups. See log for details."));
		return false;
	}
	return true;
}

static bool GenerateHairCardGenGroomData(const int PointsPerStrand, const FHairDescription& InGroomData, FHairCardGen_GroomData& Out, const FHairCardGeneratorUtils::FBuilderFunction& BuilderFunction)
{
	const int NumStrands = InGroomData.GetNumStrands();
	Out.Strands.Reserve(NumStrands);

	TGroomAttributesConstRef<float> GroomHairWidth = InGroomData.GroomAttributes().GetAttributesRef<float>(HairAttribute::Groom::Width);
	const FGroomID GroomID = FGroomID(0);
	const float Groom_DefaultWidth = GroomHairWidth.IsValid() ? GroomHairWidth[GroomID] : 0.f;

	FStrandAttributesRange TraverseStrands(InGroomData, FHairStrandAttribsConstIterator::DefaultIgnore);
	if ( !TraverseStrands.IsValid() )
		return false;

	UE_CLOGF(!TraverseStrands.First().HasGroupID(), LogHairCardGenerator, Warning, "Groom description has no physics group info, may not generate correctly");

	const FHairStrandAttribsRefProxy& FirstStrand = TraverseStrands.First();
	UE_CLOGF((!FirstStrand.HasCurveType() || !FirstStrand.HasBasisType() || FirstStrand.HasKnots()), LogHairCardGenerator, Warning, "Invalid strand curve specification, treating as linear");

	EGroomCurveType CurveType = FirstStrand.GetCurveType();
	EGroomBasisType BasisType = FirstStrand.GetBasisType();

	if (CurveType == EGroomCurveType::VariableOrder)
	{
		UE_LOGF(LogHairCardGenerator, Warning, "VariableOrder curves not supported, treating as linear");

		CurveType = EGroomCurveType::Linear;
		BasisType = EGroomBasisType::NoBasis;
	}

	Out.CurveType = StaticEnum<EGroomCurveType>()->GetNameStringByValue((int64)CurveType);
	Out.BasisType = StaticEnum<EGroomBasisType>()->GetNameStringByValue((int64)BasisType);

	Out.VertexPositions.Reserve(NumStrands * PointsPerStrand * 3);
	Out.VertexWidths.Reserve(NumStrands * PointsPerStrand);
	bool bHasAO = FirstStrand.HasVertexAO();
	if (bHasAO)
	{
		Out.VertexOcclusions.Reserve(NumStrands * PointsPerStrand);
	}

	TArray<TArray<FVector>> StrandsPositions;
	BuilderFunction(StrandsPositions);

	int32 StrandIndex = 0;
	for ( const FHairStrandAttribsRefProxy& StrandAttribs : TraverseStrands )
	{
		FHairCardGen_StrandData& OutStrandData = Out.Strands.AddDefaulted_GetRef();
		OutStrandData.GroupID = StrandAttribs.HasGroupID() ? StrandAttribs.GetGroupID() : -1;

		const float Strand_DefaultWidth = StrandAttribs.HasStrandWidth() ? StrandAttribs.GetStrandWidth() : Groom_DefaultWidth;

		TArray<FVector> VertexPositions;
		VertexPositions.SetNum(StrandAttribs.NumVerts());
		TArray<float> VertexWidths;
		VertexWidths.SetNum(StrandAttribs.NumVerts());
		TArray<float> VertexAOs;
		VertexAOs.Reserve(StrandAttribs.NumVerts());

		const bool bIsVertexValid = StrandsPositions.IsValidIndex(StrandIndex) && (StrandAttribs.NumVerts() <= StrandsPositions[StrandIndex].Num());
		for (int StrandVertIndex = 0; StrandVertIndex < StrandAttribs.NumVerts(); ++StrandVertIndex)
		{
			VertexPositions[StrandVertIndex] = bIsVertexValid ? StrandsPositions[StrandIndex][StrandVertIndex] : FVector(StrandAttribs.GetVertexPosition(StrandVertIndex));
			VertexWidths[StrandVertIndex] = StrandAttribs.HasVertexWidths() ? StrandAttribs.GetVertexWidth(StrandVertIndex) : Strand_DefaultWidth;
			if (bHasAO)
			{
				VertexAOs.Add(StrandAttribs.GetVertexAO(StrandVertIndex));
			}
		}

		FHairCardGenStrandNSCInterpolator::StrandInterpolationResult InterpolatedStrand = FHairCardGenStrandNSCInterpolator(VertexPositions, VertexWidths, VertexAOs).GetInterpolatedStrand(PointsPerStrand);

		Out.VertexPositions.Append(InterpolatedStrand.Positions);
		Out.VertexWidths.Append(InterpolatedStrand.Widths);
		Out.VertexOcclusions.Append(InterpolatedStrand.AOs);
		++StrandIndex;
	}

	Out.Strands.Shrink();
	Out.VertexPositions.Shrink();
	Out.VertexWidths.Shrink();
	Out.VertexOcclusions.Shrink();

	Out.HairlineGroupID = INDEX_NONE;

	return true;
}

void FHairCardGeneratorUtils::BuildCardsGroups(TObjectPtr<const UGroomAsset> GroomAsset, TArray<FName>& GroupNames, TArray<int32>& GroupCount)
{
	HairCardSettings_Helpers::GetStrandCardGroupInfo(GroomAsset, GroupNames, GroupCount);
}

bool FHairCardGeneratorUtils::LoadGroomStrands(const UGroomAsset* GroomAsset, const FBuilderFunction& BuilderFunction)
{
	TObjectPtr<UHairCardGenControllerBase> CardGenController = FHairCardGenController::GetHairCardGenerator();
	if ( !CardGenController )
	{
		return false;
	}
	FHairCardGen_GroomData CardGenData;
	GenerateHairCardGenGroomData(CardGenController->GetPointsPerCurve(), GroomAsset->GetHairDescription(), CardGenData, BuilderFunction);

	FString CachedGroomsPath = FPaths::ProjectIntermediateDir() / TEXT("GroomHairCardGen") / TEXT("CachedGrooms");
	return CardGenController->LoadGroomData(CardGenData, GroomAsset->GetName(), CachedGroomsPath);
}

bool FHairCardGeneratorUtils::BuildCardsAssets(UGroomAsset* NewGroomAsset, FHairGroupsCardsSourceDescription& CardsDesc,
	const TObjectPtr<UHairCardGeneratorPluginSettings>& GenerationSettings, const uint8 GenFlags)
{
	
	TObjectPtr<UHairCardGenControllerBase> CardGenController = FHairCardGenController::GetHairCardGenerator();
	if ( !CardGenController )
	{
		return false;
	}

	// Import Textures (NOTE: Mesh import no longer supported)
	if ( GenerationSettings->CheckGenerationFlags(GenFlags, EHairCardGenerationPipeline::ImportTextures) )
	{
		FHairCardImportSettings ImportSettings;
		HairCardGeneratorEditor_Impl::GetTextureImportNames(GenerationSettings, ImportSettings);

		const FString TexturePath = GenerationSettings->GetTextureImportPath();
		const FString TextureDest = GenerationSettings->GetTextureContentPath();
		UE_LOGF(LogHairCardGenerator, Display, "Importing generated cards from: %ls", *TexturePath);
		TArray<UObject*> ImportedAssets = HairCardGeneratorEditor_Impl::ImportHairCardTextures(TexturePath, TextureDest, ImportSettings);
		if ( ImportedAssets.IsEmpty() )
		{
			UE_LOGF(LogHairCardGenerator, Error, "Failed to import textures. Check '%ls' for the files.", *TexturePath);
			HairCardGeneratorEditor_Impl::CreateFinishNotification(false, LOCTEXT("HairCardGen.FailedTextureImport", "Failed to import textures. See log for details."));
			return false;
		}

		const int32 NumAssigned = HairCardGeneratorEditor_Impl::AssignImportedTextures(ImportedAssets, ImportSettings, CardsDesc);
		if ( NumAssigned == 0 )
		{
			UE_LOGF(LogHairCardGenerator, Error, "Unable to assign any of the imported assets to the groom. Verify texture file name format.");
			HairCardGeneratorEditor_Impl::CreateFinishNotification(false, LOCTEXT("HairCardGen.FailedAssignImports", "Unable to assign any of the imported assets to the groom. See log for details."));
			return false;
		}
	}
	
	// Use num groups and out group id to determine mesh name
	const int32 NumPhysGroups = NewGroomAsset->GetHairDescriptionGroups().HairGroups.Num();

	// Generate static mesh from cards and add to destination package
	if ( GenerationSettings->CheckGenerationFlags(GenFlags, EHairCardGenerationPipeline::GenerateMesh) )
	{
		FScopedSlowTask SlowTask(0, LOCTEXT("GeneratingHairCards.BuildMesh", "Building mesh..."));
		SlowTask.MakeDialog(/*bShowCancelButton =*/true);

		// Don't bother adding _PG# at end of mesh name if there's only one physics group
		int OutGroupId = (NumPhysGroups > 1) ? GenerationSettings->GetGenerateForGroomGroup() : -1;
		TObjectPtr<UStaticMesh> NewMesh = HairCardGeneratorEditor_Impl::CreateMeshInPackage(GenerationSettings->DestinationPath.Path, GenerationSettings->BaseFilename, OutGroupId);
		if ( !NewMesh )
		{
			
			UE_LOGF(LogHairCardGenerator, Error, "Unable to create mesh in destination package.");
			HairCardGeneratorEditor_Impl::CreateFinishNotification(false, LOCTEXT("HairCardGen.FailedObjectCreate", "Unable to create mesh in destination package. See log for details."));
			return false;
		}

		// Load all per-group and layout data from disk for mesh generation
		const int32 NumGroupsForMesh = GenerationSettings->GetFilterGroupSettings().Num();
		TArray<FHairCardGeomData> MeshAllGeom;
		TArray<FHairCardMeshData> MeshAllMesh;
		TArray<FHairCardTextureClusterData> MeshAllCluster;
		TArray<int32> MeshAllCardCount;
		MeshAllGeom.SetNum(NumGroupsForMesh);
		MeshAllMesh.SetNum(NumGroupsForMesh);
		MeshAllCluster.SetNum(NumGroupsForMesh);
		MeshAllCardCount.SetNum(NumGroupsForMesh);
		for (int32 i = 0; i < NumGroupsForMesh; ++i)
		{
			if (!CardGenController->LoadCardData(i, MeshAllGeom[i], MeshAllMesh[i], MeshAllCardCount[i]) ||
			    !CardGenController->LoadTextureClusterData(i, MeshAllCluster[i]))
			{
				UE_LOG(LogHairCardGenerator, Error, TEXT("Failed to load stage data for group %d in mesh generation step."), i);
				HairCardGeneratorEditor_Impl::CreateFinishNotification(false, LOCTEXT("HairCardGen.FailedMeshGen", "Mesh generation failed. See log for details."));
				return false;
			}
		}
		FHairCardAtlasLayoutData MeshLayoutData;
		if (!CardGenController->LoadAtlasLayoutData(MeshLayoutData))
		{
			UE_LOG(LogHairCardGenerator, Error, TEXT("Failed to load atlas layout data in mesh generation step."));
			HairCardGeneratorEditor_Impl::CreateFinishNotification(false, LOCTEXT("HairCardGen.FailedMeshGen", "Mesh generation failed. See log for details."));
			return false;
		}

		if ( !CardGenController->GenerateMesh(NewMesh, MeshAllGeom, MeshAllMesh, MeshAllCluster, MeshLayoutData) )
		{
			UE_LOGF(LogHairCardGenerator, Error, "Mesh generation failed. See log for details.");
			HairCardGeneratorEditor_Impl::CreateFinishNotification(false, LOCTEXT("HairCardGen.FailedMeshGen", "Mesh generation failed. See log for details."));
			return false;
		}

		CardsDesc.ImportedMesh = NewMesh;
		CardsDesc.UpdateMeshKey();

		NewMesh->PostEditChange();
		NewMesh->GetPackage()->MarkPackageDirty();
	}

	GenerationSettings->WritePipelineGenerated();
	HairCardGeneratorEditor_Impl::CreateFinishNotification(true, LOCTEXT("HairCardGen.GenSuccess", "Hair cards generated successfully."));
	return true;
}

/* HairCardGeneratorEditor_Impl Definition
 *****************************************************************************/

// Always assume the first argument is a (sometimes optional) groom argument and get UGroomAsset, if possible
static TObjectPtr<UGroomAsset> HairCardGeneratorEditor_Impl::ParseGroomAssetArg(const TArray<FString>& Args, bool bOptional)
{
	if ( Args.Num() < 1 )
		return nullptr;

	FString AssetPath = Args[0];
	FSoftObjectPath CheckPath(AssetPath);

	if ( !CheckPath.IsValid() )
	{
		UE_CLOGF(!bOptional, LogHairCardGenerator, Error, "The path is not a valid asset path.");
		return nullptr;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	FAssetData ExistingAsset = AssetRegistryModule.Get().GetAssetByObjectPath(CheckPath);
	if (!ExistingAsset.IsValid())
	{
		UE_CLOGF(!bOptional, LogHairCardGenerator, Error, "The asset path cannot be found in the registry.");
		return nullptr;
	}


	if(ExistingAsset.GetClass() != UGroomAsset::StaticClass())
	{
		UE_CLOGF(!bOptional, LogHairCardGenerator, Error, "The specified asset is not a groom asset.");
		return nullptr;
	}

	return LoadObject<UGroomAsset>(nullptr, *AssetPath);
}

static FHairGroupsCardsSourceDescription* HairCardGeneratorEditor_Impl::FindCardSourceDescription(TObjectPtr<UGroomAsset> Groom, int32 LODIndex)
{
	// Try to load previous settings from a matching (LOD) card source description slot, if compatible
	for ( FHairGroupsCardsSourceDescription& Desc : Groom->GetHairGroupsCards() )
	{
		if ( Desc.LODIndex == LODIndex)
		{
			return &Desc;
		}
	}

	return nullptr;
}

static TObjectPtr<UHairCardGeneratorPluginSettings> HairCardGeneratorEditor_Impl::CreateGroomSettingsLast(TObjectPtr<UGroomAsset> Groom, 
	const FHairGroupsCardsSourceDescription& SourceDesc, const FString& DefaultDirectory, const FString& DefaultName)
{
	TObjectPtr<UHairCardGeneratorPluginSettings> NewSettings = NewObject<UHairCardGeneratorPluginSettings>(/*Outer =*/Groom);
	NewSettings->SetSource(Groom, SourceDesc.LODIndex, SourceDesc.GroupIndex);
	NewSettings->ResetToDefault(DefaultDirectory, DefaultName);

	// First try to reset from Json file (then fallback to previous settings if they exist)
	if ( NewSettings->ResetFromSettingsJson(DefaultDirectory, DefaultName) )
		return NewSettings;

	NewSettings->ResetFromSourceDescription(SourceDesc, DefaultDirectory, DefaultName);
	return NewSettings;
}

static void HairCardGeneratorEditor_Impl::DbgCmdCacheGroom(const TArray<FString>& Args)
{
	TObjectPtr<UGroomAsset> GroomAsset = HairCardGeneratorEditor_Impl::ParseGroomAssetArg(Args, false);
	if ( !GroomAsset )
		return;

	if ( !HairCardGeneratorEditor_Impl::LoadGroomData(GroomAsset, true) )
	{
		UE_LOGF(LogHairCardGenerator, Error, "Failed to cache groom asset.");
	}
}

static bool HairCardGeneratorEditor_Impl::LoadGroomData(UGroomAsset* NewGroomAsset, const bool SaveCached)
{
	if (HairCardGeneratorEditor_Impl::LastGroom == NewGroomAsset && !SaveCached)
	{
		return true;
	}
	else
	{
		TObjectPtr<UHairCardGenControllerBase> CardGenController = FHairCardGenController::GetHairCardGenerator();
		if ( !CardGenController )
			return false;

		FHairCardGen_GroomData RestructuredData;
		if ( !HairCardGeneratorEditor_Impl::BuildHairCardGenGroomData(CardGenController->GetPointsPerCurve(), NewGroomAsset->GetHairDescription(), RestructuredData))
			return false;

		FScopedSlowTask SlowTask(0, LOCTEXT("GeneratingHairCards.LoadGroom", "Loading Groom..."));
		SlowTask.MakeDialog(/*bShowCancelButton =*/true);

		FString CachedGroomsPath = FPaths::ProjectIntermediateDir() / TEXT("GroomHairCardGen") / TEXT("CachedGrooms");
		if (!CardGenController->LoadGroomData(RestructuredData, NewGroomAsset->GetName(), CachedGroomsPath, SaveCached)) 
		{
			return false;
		}
		else
		{
			HairCardGeneratorEditor_Impl::LastGroom = NewGroomAsset;
			return true;
		}
	}
}

static bool HairCardGeneratorEditor_Impl::BuildHairCardGenGroomData(const int PointsPerStrand, const FHairDescription& InGroomData, FHairCardGen_GroomData& Out)
{
	return GenerateHairCardGenGroomData(PointsPerStrand, InGroomData, Out, [](TArray<TArray<FVector>>&){});
}

static bool HairCardGeneratorEditor_Impl::GenerateCardsGeometry(TObjectPtr<const UHairCardGeneratorPluginSettings> Settings, int index, FHairCardGeomData& OutGeomData, FHairCardMeshData& OutMeshData, int32& OutCardCount)
{
	TObjectPtr<UHairCardGenControllerBase> CardGenController = FHairCardGenController::GetHairCardGenerator();
	if ( !CardGenController )
	{
		return false;
	}

	FScopedSlowTask SlowTask(10, FText::Format(LOCTEXT("GeneratingHairCards.GeometryOptimization", "Generating cards geometry"), index));

	// Load clump data that was written by the clumps stage
	SlowTask.EnterProgressFrame(1);
	FHairCardClumpData ClumpData;
	if (!CardGenController->LoadClumpData(index, ClumpData))
	{
		UE_LOG(LogHairCardGenerator, Error, TEXT("Failed to load clump data for settings group %d. Check the log for details."), index);
		SlowTask.EnterProgressFrame(9);
		return false;
	}

	if (!CardGenController->SetOptimizations(index, ClumpData))
	{
		UE_LOGF(LogHairCardGenerator, Error, "Card geometry generation failed for settings group %d. Check the log for details.", index);
		SlowTask.EnterProgressFrame(9);
		return false;
	}

	TArray<int32> NumCards = CardGenController->GetNumCardsPerClump();

	int TotalNumCards = 0;
	for (int CurrentNumCards : NumCards) TotalNumCards += CurrentNumCards;

	// Empty group: no cards to subdivide. Leave OutGeomData/OutMeshData default-constructed
	// (all arrays empty) and OutCardCount = 0. SaveCardData will persist empty files so
	// downstream LoadCardData succeeds unconditionally for all groups.
	if (NumCards.IsEmpty() || TotalNumCards == 0)
	{
		UE_LOG(LogHairCardGenerator, Warning, TEXT("Settings group %d produced no cards — saving empty geometry data and continuing."), index);
		SlowTask.EnterProgressFrame(9);
		OutCardCount = 0;
		return true;
	}

	float AvgLength = 0;
	float AvgCurvRadius = 0;

	TArray<MatrixXf> AvgCurves;
	AvgCurves.SetNum(TotalNumCards);

	int CardId = 0;
	for (int Id = 0; Id < NumCards.Num(); Id++)
	{
		for (int Cid = 0; Cid < NumCards[Id]; Cid++)
		{
			const TArray<float> AvgCurvePoints = CardGenController->GetAverageCurve(Id, Cid);

			if (AvgCurvePoints.Num() == 0)
			{
				UE_LOGF(LogHairCardGenerator, Error, "Card geometry generation failed for settings group %d. Check the log for details.", index);
				SlowTask.EnterProgressFrame(9);
				return false;
			}

			AvgCurves[CardId] = FHairCardGenCardSubdivider::TArrayToMatrixXf(AvgCurvePoints);
			AvgLength += FHairCardGenCardSubdivider::GetCurveLength(AvgCurves[CardId]);
			AvgCurvRadius += FHairCardGenCardSubdivider::GetAverageCurvatureRadius(AvgCurves[CardId]);

			CardId++;
		}
	}

	AvgLength /= (float)TotalNumCards;
	AvgCurvRadius /= (float)TotalNumCards;

	auto GetSubdToleranceFromTriCount = [AvgLength, AvgCurvRadius, TotalNumCards](int TriCount)
	{
		const float NumSubdivisions = 0.5 * (float)TriCount / (float)TotalNumCards;
		return FHairCardGenCardSubdivider::GetToleranceFromLengthAndCurvRadius(AvgLength, AvgCurvRadius, NumSubdivisions);
	};

	int const InitialTargetTriCount = Settings->GetFilterGroupSettings(index)->TargetTriangleCount;

	FHairCardGenCardSubdivider Subdivider(
		GetSubdToleranceFromTriCount(InitialTargetTriCount),
		Settings->GetFilterGroupSettings(index)->UseAdaptiveSubdivision,
		Settings->GetFilterGroupSettings(index)->MaxVerticalSegmentsPerCard);

	if (InitialTargetTriCount <= 2 * TotalNumCards)
	{
		Subdivider.SetSubdTolerance(-1.);
	}
	else
	{
		auto CountTriangles = [&Subdivider, &AvgCurves]()
		{
			int TriCount = 0;
			for (int CardId = 0; CardId < AvgCurves.Num(); CardId++) TriCount += 2 * (Subdivider.GetSubdivisionPoints(AvgCurves[CardId]).Num() / 3 - 1);
			return TriCount;
		};

		const int InitialTriCount = CountTriangles();

		int ShiftedTargetTriCount = 2 * InitialTargetTriCount - InitialTriCount;
		int const MinTargetTriCount = 2.2 * float(TotalNumCards * InitialTargetTriCount) / float(InitialTriCount);
		if (ShiftedTargetTriCount < MinTargetTriCount) ShiftedTargetTriCount = MinTargetTriCount;

		Subdivider.SetSubdTolerance(GetSubdToleranceFromTriCount(ShiftedTargetTriCount));
		const int ShiftedTriCount = CountTriangles();

		if (InitialTriCount != ShiftedTriCount)
		{
			const int CorrectedTargetTriCount = InitialTargetTriCount + int(float(ShiftedTargetTriCount - InitialTargetTriCount) / float(ShiftedTriCount - InitialTriCount) * float(InitialTargetTriCount - InitialTriCount));
			Subdivider.SetSubdTolerance(GetSubdToleranceFromTriCount(CorrectedTargetTriCount));
		}
	}

	CardId = 0;
	for (int Id = 0; Id < NumCards.Num(); Id++)
	{
		for (int Cid = 0; Cid < NumCards[Id]; Cid++)
		{
			TArray<float> SubdividedPoints = Subdivider.GetSubdivisionPoints(AvgCurves[CardId]);
			CardGenController->SetInterpolatedAvgCurve(Id, Cid, SubdividedPoints);

			CardId++;
		}
	}

	SlowTask.EnterProgressFrame(9);
	if (!CardGenController->GenerateCardsGeometry(OutGeomData, OutMeshData, OutCardCount))
	{
		UE_LOGF(LogHairCardGenerator, Error, "Card geometry generation failed for settings group %d. Check the log for details.", index);
		return false;
	}

	return true;
}


static bool HairCardGeneratorEditor_Impl::GenerateCardsForCardGroup(const TObjectPtr<const UHairCardGeneratorPluginSettings> GenerationSettings, const int32 FilterIndex, const uint8 GenFlags)
{
	FScopedSlowTask SlowTask(10, FText::Format(LOCTEXT("GeneratingHairCards.GenerateGroup", "Generating Hair Cards for settings group {0}..."), FilterIndex));
	SlowTask.MakeDialog(/*bShowCancelButton =*/true);

	SlowTask.EnterProgressFrame(1);
	FHairCardClumpData ClumpData;
	if(!FHairCardGeneratorUtils::GenerateCardsClumps(GenerationSettings, FilterIndex, GenFlags, ClumpData))
	{
		SlowTask.EnterProgressFrame(9);
		return false;
	}

	FHairCardGeomData GeomData;
	FHairCardMeshData MeshData;
	int32 CardCount = 0;
	SlowTask.EnterProgressFrame(5);
	if(!FHairCardGeneratorUtils::GenerateCardsGeometry(GenerationSettings, FilterIndex, GenFlags, GeomData, MeshData, CardCount))
	{
		SlowTask.EnterProgressFrame(4);
		return false;
	}

	SlowTask.EnterProgressFrame(4);
	FHairCardTextureClusterData ClusterData;
	if(!FHairCardGeneratorUtils::GenerateCardsTexturesClusters(GenerationSettings, FilterIndex, GenFlags, ClusterData))
	{
		return false;
	}

	return true;
}

static void HairCardGeneratorEditor_Impl::GetTextureImportNames(UHairCardGeneratorPluginSettings* Settings, FHairCardImportSettings& ImportSettings)
{
	uint32 NumTextures = GetHairTextureLayoutTextureCount(Settings->ChannelLayout);
	FString TextureBasename = Settings->GetTextureImportBaseName();

	ImportSettings.TextureNames.SetNum(NumTextures);
	ImportSettings.MeshName = Settings->BaseFilename;

	for (uint32 i = 0; i < NumTextures; ++i)
	{
		ImportSettings.TextureNames[i] = FString::Printf(TEXT("%s_TS%d"), *TextureBasename, i);
	}
}

static TArray<UObject*> HairCardGeneratorEditor_Impl::ImportHairCardTextures(const FString& ImportSrc, const FString& ImportDst, const FHairCardImportSettings& ImportSettings)
{
	// Create a texture factory for importing tangents/attributes with linear colorspace
	UTextureFactory* LinearTextureFactory = NewObject<UTextureFactory>();
	LinearTextureFactory->ColorSpaceMode = ETextureSourceColorSpace::Linear;

	UAutomatedAssetImportData* AutoImportLinearTextures = NewObject<UAutomatedAssetImportData>();
	AutoImportLinearTextures->DestinationPath = ImportDst;
	AutoImportLinearTextures->bReplaceExisting = true;
	AutoImportLinearTextures->bSkipReadOnly = false;
	AutoImportLinearTextures->Factory = LinearTextureFactory;

	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
	FileManager.IterateDirectory(*ImportSrc, [AutoImportLinearTextures, ImportSettings](const TCHAR* FilenameOrDirectory, const bool bIsDirectory)->bool
		{
			if (!bIsDirectory)
			{
				AutoImportLinearTextures->Filenames.Add(FString(FilenameOrDirectory));
			}
			return true;
		}
	);

	TArray<UObject*> ImportedAssets = FAssetToolsModule::GetModule().Get().ImportAssetsAutomated(AutoImportLinearTextures);
	return ImportedAssets;
}

static UStaticMesh* HairCardGeneratorEditor_Impl::CreateMeshInPackage(const FString& DestinationPackage, const FString& BaseFilename, int GroupID)
{
	FString MeshName = BaseFilename;
	if ( GroupID >= 0 )
	{
		MeshName += FString::Printf(TEXT("_PG%d"), GroupID);
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	FString PackageDest = ObjectTools::SanitizeInvalidChars(FPaths::Combine(*DestinationPackage, *MeshName), INVALID_LONGPACKAGE_CHARACTERS);
	FString ObjectName = ObjectTools::SanitizeObjectName(MeshName);

	UPackage* PackageOuter = CreatePackage(*PackageDest);
	TObjectPtr<UObject> ExistingMesh = FindObject<UStaticMesh>(PackageOuter, *ObjectName);
	if ( !ExistingMesh )
	{
		UStaticMesh* NewMesh = NewObject<UStaticMesh>(PackageOuter, UStaticMesh::StaticClass(), *ObjectName, RF_Public | RF_Standalone);
		// Add asset path to registry to get folder reference in content browser
		AssetRegistryModule.Get().AssetCreated(NewMesh);

		return NewMesh;
	}

	if ( ExistingMesh->GetClass()->IsChildOf(UStaticMesh::StaticClass()) )
	{
		// Free any RHI resources for existing mesh before we re-create in place.
		ExistingMesh->PreEditChange(nullptr);
		UStaticMesh* NewMesh = NewObject<UStaticMesh>(PackageOuter, UStaticMesh::StaticClass(), *ObjectName, RF_Public | RF_Standalone);
		AssetRegistryModule.Get().AssetCreated(NewMesh);

		return NewMesh;
	}

	UE_LOGF(LogHairCardGenerator, Error, "Non-mesh asset ('%ls') already exists in package: '%ls'", *ObjectName, *PackageDest);

	return nullptr;
}

static int32 HairCardGeneratorEditor_Impl::AssignImportedTextures(const TArray<UObject*>& ImportedAssets, const FHairCardImportSettings& ImportSettings, FHairGroupsCardsSourceDescription& Assignee)
{
	int32 AssignedAssets = 0;
	for (UObject* ImportedObject : ImportedAssets)
	{
		TObjectPtr<UTexture2D> TextureImport = Cast<UTexture2D>(ImportedObject);
		if ( TextureImport )
		{
			bool bFound = false;
			int SlotIdx = ImportSettings.TextureNames.Find(TextureImport->GetName());
			if (SlotIdx != INDEX_NONE)
			{
				Assignee.Textures.SetTexture(SlotIdx, TextureImport);
				bFound = true;
				++AssignedAssets;
			}

			UE_CLOGF(!bFound, LogHairCardGenerator, Warning, "Unable to determine which slot imported texture ('%ls') is for. Check your config.", *TextureImport->GetName());
		}
		else
		{
			UE_LOGF(LogHairCardGenerator, Warning, "Unused asset imported along with hair card mesh: '%ls'", *ImportedObject->GetName());
		}
	}

	return AssignedAssets;
}

static void HairCardGeneratorEditor_Impl::CreateFinishNotification(bool bSuccess, const FText& NotificationText)
{
	FNotificationInfo Info(NotificationText);

	Info.bFireAndForget = true;
	Info.bUseLargeFont = true;
	Info.bUseSuccessFailIcons = true;

	Info.FadeOutDuration = 4.0f;
	Info.ExpireDuration = Info.FadeOutDuration;

	SNotificationItem::ECompletionState State = (bSuccess) ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail;

	TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
	NotificationPtr->SetCompletionState(State);
}

/* FHairCardGeneratorEditorModule
 *****************************************************************************/

void FHairCardGeneratorEditorModule::StartupModule()
{
	LLM_SCOPE_BYTAG(HairCardGenerator)

	HairCardGenerator_Utils::RegisterModularHairCardGenerator(this);

	// Register custom detail panel handler for hair card settings
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(UHairCardGeneratorPluginSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FHairCardSettingsDetailCustomization::MakeInstance));
}

void FHairCardGeneratorEditorModule::ShutdownModule()
{
	HairCardGenerator_Utils::UnregisterModularHairCardGenerator(this);

	// Register custom detail panel handler for hair card settings
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.UnregisterCustomClassLayout(UHairCardGeneratorPluginSettings::StaticClass()->GetFName());
}

bool FHairCardGeneratorEditorModule::GenerateHairCardsForLOD(UGroomAsset* NewGroomAsset, FHairGroupsCardsSourceDescription& CardsDesc)
{
	TObjectPtr<UHairCardGeneratorPluginSettings> GenerationSettings = nullptr;
	uint8 AllGenFlags = 0, PipelineGenFlags = 0;
	if(!FHairCardGeneratorUtils::BuildGenerationSettings(true, NewGroomAsset, CardsDesc, GenerationSettings, AllGenFlags, PipelineGenFlags, FCardsGenerationAdvancedOptions()))
	{
		return false;
	}

	if (!HairCardGeneratorEditor_Impl::LoadGroomData(NewGroomAsset))
	{
		UE_LOGF(LogHairCardGenerator, Error, "Failed to load groom asset. See log for details.");
		HairCardGeneratorEditor_Impl::CreateFinishNotification(false, LOCTEXT("HairCardGen.FailedGroomLoad", "Failed to load groom asset. See log for details."));
		return false;
	}
	
	if(GenerationSettings)
	{
		GenerationSettings->UpdateStrandFilterAssignment();
	}

	if(!FHairCardGeneratorUtils::LoadGenerationSettings(GenerationSettings))
	{
		return false;
	}
	
	// Find the HairCardGenController in Python (or wherever it may be implemented -- could be moved to C++/Blueprints/etc.)
	TObjectPtr<UHairCardGenControllerBase> CardGenController = FHairCardGenController::GetHairCardGenerator();
	if ( !CardGenController )
	{
		UE_LOGF(LogHairCardGenerator, Error, "No hair card generator controller found, cannot generate hair cards!");
		return false;
	}

	if(!FHairCardGeneratorUtils::RunCardsGeneration(GenerationSettings, PipelineGenFlags,
		[](const TObjectPtr<const UHairCardGeneratorPluginSettings>& GenerationSettings, const int32 FilterIndex, const uint8 GenFlags)
		{
			return HairCardGeneratorEditor_Impl::GenerateCardsForCardGroup(GenerationSettings, FilterIndex, GenFlags);
		}))
	{
		return false;
	}

	FHairCardAtlasLayoutData LayoutData;
	FHairCardAtlasUVData AtlasUVData;
	if(!FHairCardGeneratorUtils::GenerateTexturesLayoutAndAtlases(GenerationSettings, AllGenFlags, LayoutData, AtlasUVData))
	{
		return false;
	}

	if(!FHairCardGeneratorUtils::BuildCardsAssets(NewGroomAsset, CardsDesc, GenerationSettings, AllGenFlags))
	{
		return false;
	}
	return true;
}

bool FHairCardGeneratorEditorModule::IsCompatibleSettings(UHairCardGenerationSettings* OldSettings)
{
	return UHairCardGeneratorPluginSettings::IsCompatibleSettings(OldSettings);
}

IMPLEMENT_MODULE(/*FDefaultModuleImpl*/FHairCardGeneratorEditorModule, HairCardGeneratorEditor);
#undef LOCTEXT_NAMESPACE
