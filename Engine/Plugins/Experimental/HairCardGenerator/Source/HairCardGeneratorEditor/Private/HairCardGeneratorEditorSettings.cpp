// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardGeneratorEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HairCardGeneratorEditorSettings)

UHairCardGeneratorEditorSettings::UHairCardGeneratorEditorSettings()
	: HairCardAssetNameFormat(TEXT("{groom_name}_LOD{lod}"))
	, HairCardAssetPathFormat(TEXT("{groom_dir}"))
{
}
