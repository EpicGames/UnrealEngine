// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "HairCardGeneratorEditorSettings.generated.h"

enum class EHairAtlasTextureType : uint8;


UCLASS(config = Editor)
class UHairCardGeneratorEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	UHairCardGeneratorEditorSettings();

	UPROPERTY(Config)
	FString HairCardAssetNameFormat;

	UPROPERTY(Config)
	FString HairCardAssetPathFormat;
};