// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

HAIRCARDGENERATOREDITOR_API DECLARE_LOG_CATEGORY_EXTERN(LogHairCardGenerator, Log, All)