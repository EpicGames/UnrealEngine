// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardGeneratorPluginSettings.h"

#include "HairCardGeneratorEditorSettings.h"
#include "HairCardGeneratorLog.h"
#include "HairCardGenCardSubdivider.h"

#include "GroomAsset.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Dom/JsonObject.h"
#include "Engine/StaticMesh.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "ObjectTools.h"
#include "JsonObjectConverter.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"


#define LOCTEXT_NAMESPACE "HairCardPluginSettings"

/* Json helper functions
 *****************************************************************************/
namespace FJsonHelper
{
	// TODO: Need to cache settings with IDirectoryWatcher or FFileCache to avoid hitting fs constantly in UI (similar issue: HCS settings change detection)
	TSharedPtr<FJsonObject> ReadJsonSettings(const FString& SettingsFilename);

	FString GetStringFieldDefault(const TSharedPtr<FJsonObject>& Json, const FString& FieldName, const FString& DefaultVal = TEXT(""));
	bool GetBoolFieldDefault(const TSharedPtr<FJsonObject>& Json, const FString& FieldName, bool DefaultVal = false);
	int GetIntFieldDefault(const TSharedPtr<FJsonObject>& Json, const FString& FieldName, int DefaultVal = 0);
};

TSharedPtr<FJsonObject> FJsonHelper::ReadJsonSettings(const FString& SettingsFilename)
{
	TSharedPtr<FJsonObject> JsonSettingsObject;

	FString JsonText;
	if ( !FFileHelper::LoadFileToString(JsonText, *SettingsFilename) )
	{
		// UE_LOGF(LogHairCardGenerator, Warning, "Unable to open json settings file: %ls", *SettingsFilename);
		return nullptr;
	}

	TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(JsonText);
	if ( !FJsonSerializer::Deserialize(JsonReader.Get(), JsonSettingsObject) )
	{
		// UE_LOGF(LogHairCardGenerator, Warning, "Unable to parse json settings file: %ls", *SettingsFilename);
		return nullptr;
	}

	return JsonSettingsObject;
}

FString FJsonHelper::GetStringFieldDefault(const TSharedPtr<FJsonObject>& Json, const FString& FieldName, const FString& DefaultVal)
{
	if (!Json->HasTypedField<EJson::String>(FieldName))
	{
		return DefaultVal;
	}

	return Json->GetStringField(FieldName);
}

bool FJsonHelper::GetBoolFieldDefault(const TSharedPtr<FJsonObject>& Json, const FString& FieldName, bool DefaultVal)
{
	if (!Json->HasTypedField<EJson::Boolean>(FieldName))
	{
		return DefaultVal;
	}

	return Json->GetBoolField(FieldName);
}

int FJsonHelper::GetIntFieldDefault(const TSharedPtr<FJsonObject>& Json, const FString& FieldName, int DefaultVal)
{
	if (!Json->HasTypedField<EJson::Number>(FieldName))
	{
		return DefaultVal;
	}

	return Json->GetIntegerField(FieldName);
}


/* Hair card helpers
 *****************************************************************************/
namespace HairCardSettings_Helpers
{
	void ResetSettingsFromTemplate(TObjectPtr<UObject> SettingsObject, TObjectPtr<const UObject> TemplateObject);
	uint8 PipelineSettingsDiff(TObjectPtr<const UObject> SettingsObject, TObjectPtr<UObject> DiffObject);
	bool IsWriteableSettingsProperty(FProperty* Property);
};


/// Get default base name for card generation (assets/directories) from current groom asset
/// @param GroomAsset The groom asset cards will be generated for
/// @param LODIndex The LOD index for card generation (TODO: This should be removed when multi-lod supported)
/// @return Base name for generated data (e.g. assets and intermediate data)
FString HairCardSettings_Helpers::GetDefaultBaseNameForHairCards(TObjectPtr<const UGroomAsset> GroomAsset, int LODIndex, const FString& DefaultName)
{
    const UHairCardGeneratorEditorSettings* ConfigSettings = GetDefault<UHairCardGeneratorEditorSettings>();

    FFormatNamedArguments FormatArgs;
	if (!DefaultName.IsEmpty())
	{
		FormatArgs.Add(TEXT("groom_name"), FText::FromString(DefaultName));
	}
	else if (GroomAsset)
    {
        FormatArgs.Add(TEXT("groom_name"), FText::FromString(GetGroomNameForHairCards(GroomAsset)));
    }
    FormatArgs.Add(TEXT("lod"), LODIndex);

    return FText::Format(FText::FromString(ConfigSettings->HairCardAssetNameFormat), FormatArgs).ToString();
}


/// Get default output content path for card generation from current groom asset
/// @param GroomAsset The groom asset cards will be generated for
/// @param LODIndex The LOD index for card generation (TODO: This should be removed when multi-lod supported)
/// @return Content path for generated hair card assets
FString HairCardSettings_Helpers::GetDefaultDestinationPathForHairCards(TObjectPtr<const UGroomAsset> GroomAsset, int LODIndex, const FString& DefaultDirectory, const FString& DefaultName)
{
	const UHairCardGeneratorEditorSettings* ConfigSettings = GetDefault<UHairCardGeneratorEditorSettings>();

	FFormatNamedArguments FormatArgs;
	if (!DefaultDirectory.IsEmpty())
	{
		FormatArgs.Add(TEXT("groom_dir"), FText::FromString(DefaultDirectory));
	}
	else
	{
		FormatArgs.Add(TEXT("groom_dir"), FText::FromString(GetGroomDirectoryForHairCards(GroomAsset)));
	}
	
	if (!DefaultName.IsEmpty())
	{
		FormatArgs.Add(TEXT("groom_name"), FText::FromString(DefaultName));
	}
	else if (GroomAsset)
    {
        FormatArgs.Add(TEXT("groom_name"), FText::FromString(GetGroomNameForHairCards(GroomAsset)));
    }
    FormatArgs.Add(TEXT("lod"), LODIndex);

    FString DirPath = FText::Format(FText::FromString(ConfigSettings->HairCardAssetPathFormat), FormatArgs).ToString();
    FPaths::NormalizeDirectoryName(DirPath);
    FPaths::RemoveDuplicateSlashes(DirPath);

    return DirPath;
}

FString HairCardSettings_Helpers::GetGroomDirectoryForHairCards(TObjectPtr<const UGroomAsset> GroomAsset)
{
	if (GroomAsset)
	{
		FString PackageRoot, PackagePath, PackageName;
		FPackageName::SplitLongPackageName(FPackageName::ObjectPathToPackageName(GroomAsset->GetPathName()),
			PackageRoot, PackagePath, PackageName);
		
		return PackageRoot + PackagePath;
	}
	return FPaths::ProjectContentDir();
}

FString HairCardSettings_Helpers::GetGroomNameForHairCards(TObjectPtr<const UGroomAsset> GroomAsset)
{
	return GroomAsset ? GroomAsset->GetName() : FString();
}

void HairCardSettings_Helpers::ResetSettingsFromTemplate(TObjectPtr<UObject> SettingsObject, TObjectPtr<const UObject> TemplateObject)
{
	if ( !TemplateObject->IsA(SettingsObject->GetClass()) )
	{
		UE_LOGF(LogHairCardGenerator, Warning, "Invalid template object for copying settings (%ls)", *SettingsObject->GetName());
		return;
	}

	for (TFieldIterator<FProperty> PropertyIt(SettingsObject->GetClass(), EFieldIterationFlags::IncludeDeprecated); PropertyIt; ++PropertyIt)
	{
		if ( !HairCardSettings_Helpers::IsWriteableSettingsProperty(*PropertyIt) )
			continue;

		const void* SrcAddress = PropertyIt->ContainerPtrToValuePtr<void>(TemplateObject);
		void* DstAddress = PropertyIt->ContainerPtrToValuePtr<void>(SettingsObject);

		PropertyIt->CopyCompleteValue(DstAddress, SrcAddress);
	}
}

uint8 HairCardSettings_Helpers::PipelineSettingsDiff(TObjectPtr<const UObject> SettingsObject, TObjectPtr<UObject> DiffObject)
{
	if ( !DiffObject->IsA(SettingsObject->GetClass()) )
	{
		UE_LOGF(LogHairCardGenerator, Warning, "Comparing settings objects of incompatible types (%ls,%ls)", *SettingsObject->GetClass()->GetName(), *DiffObject->GetClass()->GetName());
		return (uint8)(EHairCardGenerationSettingsCategories::All);
	}

	uint8 PipelineDiff = (uint8)EHairCardGenerationSettingsCategories::None;

	const UEnum* HairCardSettingsCategoriesEnum = StaticEnum<EHairCardGenerationSettingsCategories>();
	for (TFieldIterator<FProperty> PropertyIt(SettingsObject->GetClass(), EFieldIterationFlags::IncludeDeprecated); PropertyIt; ++PropertyIt)
	{
		if ( !HairCardSettings_Helpers::IsWriteableSettingsProperty(*PropertyIt) )
			continue;

		if ( !PropertyIt->HasMetaData(TEXT("Category")) )
		{
			UE_LOGF(LogHairCardGenerator, Warning, "Hair Card Settings field: %ls has no category metadata (required for change detection)!", *PropertyIt->GetName());
			continue;
		}

		FString BaseCategoryName = PropertyIt->GetMetaData(TEXT("Category"));

		// We use category names to determine pipeline effects of a difference
		BaseCategoryName.RemoveSpacesInline();
		int32 SubCatIdx = BaseCategoryName.Find(TEXT("|"));
		if ( SubCatIdx != INDEX_NONE )
			BaseCategoryName.LeftInline(SubCatIdx);

		int64 CategoryPipelineDiff = HairCardSettingsCategoriesEnum->GetValueByNameString(BaseCategoryName);
		if ( CategoryPipelineDiff == INDEX_NONE )
		{
			UE_LOGF(LogHairCardGenerator, Warning, "Hair Card Settings field: %ls has unexpected category \"%ls\" (cannot detect pipeline change effects)!", *PropertyIt->GetName(), *BaseCategoryName);
			continue;
		}

		const void* SettingsValueAddr = PropertyIt->ContainerPtrToValuePtr<void>(SettingsObject);
		const void* DiffValueAddr = PropertyIt->ContainerPtrToValuePtr<void>(DiffObject);

		if ( !PropertyIt->Identical(SettingsValueAddr, DiffValueAddr) )
			PipelineDiff |= CategoryPipelineDiff;
	}

	return PipelineDiff;
}

bool HairCardSettings_Helpers::IsWriteableSettingsProperty(FProperty* Property)
{
	// Determine if this is a settings property that should be written/compared
	// Ignore UObject properties that are private, protected, or transient
	if ( Property->HasAnyPropertyFlags(CPF_NativeAccessSpecifierPrivate | CPF_NativeAccessSpecifierProtected | CPF_Transient) )
	{
		return false;
	}
	
	// Property must be BlueprintVisible (accessible from Python)
	return Property->HasAllPropertyFlags(CPF_BlueprintVisible);
}

void HairCardSettings_Helpers::GetStrandCardGroupInfo(TObjectPtr<const UGroomAsset> GroomAsset, TArray<FName>& OutGrpIds, TArray<int32>& OutGrpCount)
{
    TMap<FName,int32> IdNames;

    FHairDescription HairDesc = GroomAsset->GetHairDescription();
	FStrandAttributesRange TraverseStrands(HairDesc, FHairStrandAttribsConstIterator::IgnoreNone);
    if ( TraverseStrands.First().HasCardGroupName() )
    {
		for ( const FHairStrandAttribsRefProxy& StrandAttribs : TraverseStrands )
		{
			int32* NumStrands = IdNames.Find(StrandAttribs.GetCardGroupName());
			if(!NumStrands)
			{
				IdNames.Add(StrandAttribs.GetCardGroupName(), 1);
			}
			else
			{
				(*NumStrands) += 1;
			}
		}

        // IdCountMap.KeySort([](FName a, FName b){return a < b;});
    }

	if ( IdNames.Contains(NAME_None) )
	{
		IdNames.Remove(NAME_None);
		UE_LOGF(LogHairCardGenerator, Warning, "Groom has groom_group_cards_id attribute, but some strands are empty (will be ignored)");
	}

	IdNames.GenerateKeyArray(OutGrpIds);
	IdNames.GenerateValueArray(OutGrpCount);
}


/* HairCardSettings_Converter
 *****************************************************************************/
namespace HairCardSettings_Converter
{
	bool ResetSettingsFromJson(UObject* SettingsObject, const TSharedPtr<FJsonObject>& JsonObject);
	bool WriteSettingsToJson(UObject* SettingsObject, TSharedRef<FJsonObject> JsonRoot);
}

bool HairCardSettings_Converter::ResetSettingsFromJson(UObject* SettingsObject, const TSharedPtr<FJsonObject>& JsonObject)
{
	for (TFieldIterator<FProperty> PropertyIt(SettingsObject->GetClass(), EFieldIterationFlags::IncludeDeprecated); PropertyIt; ++PropertyIt)
	{
		if ( !HairCardSettings_Helpers::IsWriteableSettingsProperty(*PropertyIt) )
			continue;

		TSharedPtr<FJsonValue> PropValue = JsonObject->TryGetField(PropertyIt->GetName());
		if ( !PropValue )
		{
			UE_LOGF(LogHairCardGenerator, Warning, "Json Field Not Found: %ls", *PropertyIt->GetName());
			continue;
		}

		void* SettingsPropPtr = PropertyIt->ContainerPtrToValuePtr<void>(SettingsObject);
		FJsonObjectConverter::JsonValueToUProperty(PropValue, *PropertyIt, SettingsPropPtr);
	}

	return true;
}

bool HairCardSettings_Converter::WriteSettingsToJson(UObject* SettingsObject, TSharedRef<FJsonObject> JsonRoot)
{
	// Convert all blueprint-accessible non-transient/private/protected properties
	for (TFieldIterator<FProperty> PropertyIt(SettingsObject->GetClass(), EFieldIterationFlags::IncludeDeprecated); PropertyIt; ++PropertyIt)
	{
		if ( !HairCardSettings_Helpers::IsWriteableSettingsProperty(*PropertyIt) )
			continue;

		void* SettingsPropPtr = PropertyIt->ContainerPtrToValuePtr<void>(SettingsObject);
		TSharedPtr<FJsonValue> JsonValue = FJsonObjectConverter::UPropertyToJsonValue(*PropertyIt, SettingsPropPtr);
		JsonRoot->SetField(PropertyIt->GetName(), JsonValue);
	}

	return true;
}



/* UHairCardGeneratorPluginSettings
 *****************************************************************************/

void UHairCardGeneratorPluginSettings::SetSource(TObjectPtr<UGroomAsset> InSourceObject, int32 GenLODIndex, int32 PhysGroupIndex)
{
	GroomAsset = InSourceObject;
	LODIndex = GenLODIndex;
	GenerateForGroomGroup = PhysGroupIndex;
}

void UHairCardGeneratorPluginSettings::ResetFromSourceDescription(const FHairGroupsCardsSourceDescription& SourceDesc, const FString& DefaultDirectory, const FString& DefaultName)
{
	TObjectPtr<const UHairCardGeneratorPluginSettings> LastSettings = Cast<UHairCardGeneratorPluginSettings>(SourceDesc.GenerationSettings);

	LODIndex = SourceDesc.LODIndex;
	GenerateForGroomGroup = SourceDesc.GroupIndex;
	OldGeneratedMesh = SourceDesc.ImportedMesh;

	if ( LastSettings == nullptr )
	{
		LastSettings = UHairCardGeneratorPluginSettings::StaticClass()->GetDefaultObject<UHairCardGeneratorPluginSettings>();
	}

	HairCardSettings_Helpers::ResetSettingsFromTemplate(this, LastSettings);
	if (OldGeneratedMesh && LODIndex == LastSettings->LODIndex)
	{
		BaseFilename = OldGeneratedMesh->GetName();
		DestinationPath.Path = FPackageName::GetLongPackagePath(OldGeneratedMesh->GetOuter()->GetPathName());
	}
	else
	{
		BaseFilename = HairCardSettings_Helpers::GetDefaultBaseNameForHairCards(GroomAsset, LODIndex, DefaultName);
		DestinationPath.Path = HairCardSettings_Helpers::GetDefaultDestinationPathForHairCards(GroomAsset, LODIndex, DefaultDirectory, DefaultName);
	}

	TArray<int32> CardGroupCount;
	HairCardSettings_Helpers::GetStrandCardGroupInfo(GroomAsset, CardGroupIds, CardGroupCount);

	int GroupSettingsCount = FMath::Max(1, LastSettings->FilterGroupGenerationSettings.Num());
	for ( int i=0; i < GroupSettingsCount; ++i )
	{
		TObjectPtr<UHairCardGeneratorGroupSettings> oldOpts;
		if ( LastSettings->FilterGroupGenerationSettings.Num() > i )
		{
			oldOpts = LastSettings->FilterGroupGenerationSettings[i];
		}
		else
		{
			oldOpts = UHairCardGeneratorGroupSettings::StaticClass()->GetDefaultObject<UHairCardGeneratorGroupSettings>();
		}

		ResetFilterGroupSettings(i, oldOpts);
	}
	ResetNumFilterGroups(GroupSettingsCount);

	PostResetUpdates();
}

bool UHairCardGeneratorPluginSettings::ResetFromSettingsJson(const FString& DefaultDirectory, const FString& DefaultName)
{
	// HACK: Need to bootstrap base filename and intermediate path in order to get json path
	if ( BaseFilename.IsEmpty() )
	{
		BaseFilename = HairCardSettings_Helpers::GetDefaultBaseNameForHairCards(GroomAsset, LODIndex, DefaultName);
		DestinationPath.Path = HairCardSettings_Helpers::GetDefaultDestinationPathForHairCards(GroomAsset, LODIndex, DefaultDirectory, DefaultName);
		UpdateOutputPaths();
		UpdateParentInfo();
	}

	const FString SettingsFile = GetSettingsFilename();
	if ( !FPaths::FileExists(SettingsFile) )
		return false;

	ResetFromJson();
	PostResetUpdates();

	return true;
}


void UHairCardGeneratorPluginSettings::ResetToDefault(const FString& DefaultDirectory, const FString& DefaultName)
{
	TObjectPtr<UHairCardGeneratorPluginSettings> DefaultSettings = UHairCardGeneratorPluginSettings::StaticClass()->GetDefaultObject<UHairCardGeneratorPluginSettings>();
	HairCardSettings_Helpers::ResetSettingsFromTemplate(this, DefaultSettings);

	BaseFilename = HairCardSettings_Helpers::GetDefaultBaseNameForHairCards(GroomAsset, LODIndex, DefaultName);
	DestinationPath.Path = HairCardSettings_Helpers::GetDefaultDestinationPathForHairCards(GroomAsset, LODIndex, DefaultDirectory, DefaultName);

	TArray<int32> CardGroupCount;
	HairCardSettings_Helpers::GetStrandCardGroupInfo(GroomAsset, CardGroupIds, CardGroupCount);

	ResetFilterGroupSettings(0, UHairCardGeneratorGroupSettings::StaticClass()->GetDefaultObject<UHairCardGeneratorGroupSettings>());
	ResetNumFilterGroups(1);

	PostResetUpdates();
}

void UHairCardGeneratorPluginSettings::ResetFilterGroupSettingsToDefault(int FilterGroupIndex)
{
	TObjectPtr<UHairCardGeneratorGroupSettings> DefGrpSettings = UHairCardGeneratorGroupSettings::StaticClass()->GetDefaultObject<UHairCardGeneratorGroupSettings>();
	ResetFilterGroupSettings(FilterGroupIndex, DefGrpSettings);
}

void UHairCardGeneratorPluginSettings::AddNewSettingsFilterGroup()
{
	// TODO: Remove this check when more general filter support is added to UI
	if ( FilterGroupGenerationSettings.Num() >= CardGroupIds.Num() )
		return;

	// Add a new default settings group
	ResetFilterGroupSettings(FilterGroupGenerationSettings.Num(), UHairCardGeneratorGroupSettings::StaticClass()->GetDefaultObject<UHairCardGeneratorGroupSettings>());
}

void UHairCardGeneratorPluginSettings::RemoveSettingsFilterGroup(TObjectPtr<UHairCardGeneratorGroupSettings> SettingsGroup)
{
	// Always keep one settings group
	if ( FilterGroupGenerationSettings.Num() <= 1 )
		return;

	int RemoveIndex = FilterGroupGenerationSettings.Find(SettingsGroup);
	if ( RemoveIndex != INDEX_NONE )
		FilterGroupGenerationSettings.RemoveAt(RemoveIndex);
}

void UHairCardGeneratorPluginSettings::WriteGenerationSettings()
{
	const FString SettingsPath = GetMetadataPath();

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	PlatformFile.CreateDirectoryTree(*SettingsPath);

	WriteToJson();
}

bool UHairCardGeneratorPluginSettings::CheckGenerationFlags(uint8 NeedsGenFlags, const EHairCardGenerationPipeline PipelineStage)
{
	return ((NeedsGenFlags & (uint8)PipelineStage) != 0);
}

uint8 UHairCardGeneratorPluginSettings::GetAllPipelineGeneratedDifferences() const
{
	uint8 PipelineFlags = GetPipelineGeneratedDifferences();
	if (PipelineFlags == (uint8)EHairCardGenerationSettingsCategories::All)
		return PipelineFlags;

	for ( int i=0; i < FilterGroupGenerationSettings.Num(); ++i )
		PipelineFlags |= FilterGroupGenerationSettings[i]->GetPipelineFilterGroupDifferences();

	return PipelineFlags;
}

uint8 UHairCardGeneratorPluginSettings::GetPipelineGeneratedDifferences() const
{
	if ( bForceRegen )
	{
		return (uint8)EHairCardGenerationPipeline::All;
	}

	const FString SettingsBinFile = GetGeneratedSettingsFilename();
	if ( !FPaths::FileExists(SettingsBinFile) )
	{
		return (uint8)EHairCardGenerationPipeline::All;
	}

	TArray<uint8> SettingsBytes;
	if ( !FFileHelper::LoadFileToArray(SettingsBytes, *SettingsBinFile, 0) )
	{
		return (uint8)EHairCardGenerationPipeline::All;
	}

	FMemoryReader MemReader(SettingsBytes, true);

	TObjectPtr<UHairCardGeneratorPluginSettings> DiffObject = NewObject<UHairCardGeneratorPluginSettings>();
	DiffObject->SerializeEditableSettings(MemReader);
	DiffObject->BaseFilename = BaseFilename;

	// Early-out if HCS tool versions differ from generated settings
	if (DiffObject->Version != Version)
	{
		return (uint8)EHairCardGenerationSettingsCategories::All;
	}

	return HairCardSettings_Helpers::PipelineSettingsDiff(this, DiffObject);
}

void UHairCardGeneratorPluginSettings::ClearPipelineGenerated() const
{
	const FString SettingsBinFile = GetGeneratedSettingsFilename();

	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
	FileManager.DeleteFile(*SettingsBinFile);
}

void UHairCardGeneratorPluginSettings::WritePipelineGenerated()
{
	const FString GeneratedPath = GetGeneratedSettingsPath();
	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
	if ( !FileManager.DirectoryExists(*GeneratedPath) )
		FileManager.CreateDirectoryTree(*GeneratedPath);

	TArray<uint8> SettingsBytes;
	SettingsBytes.Reserve(64);
	FMemoryWriter MemWriter(SettingsBytes, true);

	SerializeEditableSettings(MemWriter);

	const FString SettingsBinFile = GetGeneratedSettingsFilename();
	FFileHelper::SaveArrayToFile(SettingsBytes, *SettingsBinFile);
}

uint8 UHairCardGeneratorPluginSettings::GetPipelineFilterGroupDifferences(int FilterGroupIndex) const
{
	if ( FilterGroupIndex > FilterGroupGenerationSettings.Num() )
		return (uint8)EHairCardGenerationPipeline::All;

	return FilterGroupGenerationSettings[FilterGroupIndex]->GetPipelineFilterGroupDifferences();
}

void UHairCardGeneratorPluginSettings::ClearPipelineGeneratedFilterGroup(int FilterGroupIndex) const
{
	if ( FilterGroupIndex > FilterGroupGenerationSettings.Num() )
		return;

	FilterGroupGenerationSettings[FilterGroupIndex]->ClearPipelineGeneratedFilterGroup();
}

void UHairCardGeneratorPluginSettings::WritePipelineGeneratedFilterGroup(int FilterGroupIndex)
{
	if ( FilterGroupIndex > FilterGroupGenerationSettings.Num() )
		return;

	FilterGroupGenerationSettings[FilterGroupIndex]->WritePipelineGeneratedFilterGroup();
}

void UHairCardGeneratorPluginSettings::PostResetUpdates()
{
	UpdateOutputPaths();
	UpdateParentInfo();
	UpdateHairWidths();
	UpdateChannelLayout();

	EnforceValidLODSettings();
}

void UHairCardGeneratorPluginSettings::UpdateOutputPaths()
{
	IntermediatePath = GetIntermediatePath();
	OutputPath = GetIntermediatePath() / TEXT("Output") / BaseFilename;
	for ( int index = 0; index < FilterGroupGenerationSettings.Num(); ++index )
		FilterGroupGenerationSettings[index]->UpdateGenerateFilename(BaseFilename, index);
}

void UHairCardGeneratorPluginSettings::UpdateChannelLayout()
{
	ChannelLayout = EHairTextureLayout::Layout0;

	bool bFoundDesc = false;
	bool bAllMatched = true;
	EHairTextureLayout CheckLayout =  EHairTextureLayout::Layout0;
	for (const FHairGroupsCardsSourceDescription& Desc : GroomAsset->GetHairGroupsCards())
	{
		if (Desc.LODIndex != LODIndex)
		{
			continue;
		}

		if (!bFoundDesc)
		{
			bFoundDesc = true;
			CheckLayout = Desc.Textures.Layout;
		}

		if (Desc.GroupIndex == GenerateForGroomGroup)
		{
			ChannelLayout = Desc.Textures.Layout;
		}

		bAllMatched = bAllMatched & (CheckLayout == Desc.Textures.Layout);
		CheckLayout = Desc.Textures.Layout;
	}

	bValidChannelLayouts = bFoundDesc & bAllMatched;
}

void UHairCardGeneratorPluginSettings::UpdateParentInfo()
{
	BaseParentName = TEXT("");
	// Always update parent names for LOD > 0 since may need to pull reserve texture space
	if ( LODIndex >= 1 )
	{
		// TODO: Potentially allow directly specifying derived name
		FString CurLOD = FString::Printf(TEXT("_LOD%d"), LODIndex);
		FString PrevLOD = FString::Printf(TEXT("_LOD%d"), LODIndex-1);
		BaseParentName = BaseFilename.Replace(*CurLOD,*PrevLOD, ESearchCase::CaseSensitive);
	}

	FindDerivedTextureSettings();

	for ( int index = 0; index < FilterGroupGenerationSettings.Num(); ++index )
		FilterGroupGenerationSettings[index]->UpdateParentName(BaseParentName, index);
}

void UHairCardGeneratorPluginSettings::UpdateHairWidths()
{
	int const NumPhysGroups = GroomAsset->GetHairDescriptionGroups().HairGroups.Num();

	HairWidths.Init(-1.0, NumPhysGroups);
	RootScales.Init(1.0, NumPhysGroups);
	TipScales.Init(1.0, NumPhysGroups);
	if ( bUseGroomAssetStrandWidth )
	{
		for ( int PhysGroupIdx = 0; PhysGroupIdx<NumPhysGroups; PhysGroupIdx++ )
		{
			const FHairGeometrySettings& GeometrySettings = GroomAsset->GetHairGroupsRendering()[PhysGroupIdx].GeometrySettings;

			bOverrideHairWidth = GeometrySettings.HairWidth_Override;

			if ( bOverrideHairWidth )
			{
				HairWidths[PhysGroupIdx] = GeometrySettings.HairWidth;
			}
			RootScales[PhysGroupIdx] = GeometrySettings.HairRootScale;
			TipScales[PhysGroupIdx] = GeometrySettings.HairTipScale;
		}
	}
}

void UHairCardGeneratorPluginSettings::EnforceValidLODSettings()
{
	// Force these settings to be valid in case reloading settings changed something
	// TODO: Rework all of this to use a more robust json settings/completion framework
	bReduceCardsFromPreviousLOD &= CanReduceFromLOD();
	bUseReservedSpaceFromPreviousLOD &= CanReduceFromLOD();
}


bool UHairCardGeneratorPluginSettings::FindDerivedTextureSettings()
{
	DerivedTextureSettingsName = TEXT("");
	bool bUsePreviousTexture = bReduceCardsFromPreviousLOD || bUseReservedSpaceFromPreviousLOD;
	if ( !bUsePreviousTexture )
	{
		DerivedTextureSettingsName = BaseFilename;
		return true;
	}

	TSharedPtr<FJsonObject> ParentTextureSettings = GetParentTextureSettings();
	if ( !ParentTextureSettings.IsValid() )
	{
		return false;
	}

	DerivedTextureSettingsName = FJsonHelper::GetStringFieldDefault(ParentTextureSettings, TEXT("BaseFilename"));
	return true;
}

TSharedPtr<FJsonObject> UHairCardGeneratorPluginSettings::GetParentTextureSettings() const
{
	return TraverseSettingsParents(BaseParentName, 
		[](const TSharedPtr<FJsonObject>& SettingsRoot)
		{
			bool bDerivedGeometry = FJsonHelper::GetBoolFieldDefault(SettingsRoot, TEXT("bReduceCardsFromPreviousLOD"));
			bool bDerivedTexture = FJsonHelper::GetBoolFieldDefault(SettingsRoot, TEXT("bUseReservedSpaceFromPreviousLOD"));
			return !bDerivedGeometry && !bDerivedTexture;
		}
	);
}

TSharedPtr<FJsonObject> UHairCardGeneratorPluginSettings::TraverseSettingsParents(const FString& SettingsName, TFunction<bool(const TSharedPtr<FJsonObject>&)> StopCond)
{
	TSet<FString> CycleBreak;
	FString PrevLOD = SettingsName;

	// Break infinite loops with a max lod
	const int MaxLOD = 15;
	for (int i=0; i < MaxLOD && !PrevLOD.IsEmpty(); ++i)
	{
		const FString SettingsFilename = GetSettingsFileFromBase(PrevLOD);
		TSharedPtr<FJsonObject> SettingsRoot = FJsonHelper::ReadJsonSettings(SettingsFilename);
		if ( !SettingsRoot.IsValid() )
		{
			return nullptr;
		}

		// Check if this setting meets stop criterion
		if ( StopCond(SettingsRoot) )
		{
			return SettingsRoot;
		}

		// Track settings names in case there's a cycle
		if ( CycleBreak.Contains(PrevLOD) )
		{
			return nullptr;
		}

		CycleBreak.Add(PrevLOD);
		PrevLOD = FJsonHelper::GetStringFieldDefault(SettingsRoot, TEXT("PreviousLOD"));
	}

	return nullptr;
}


FString UHairCardGeneratorPluginSettings::GetIntermediatePath()
{
	return FPaths::ProjectIntermediateDir() / TEXT("GroomHairCardGen");
}

FString UHairCardGeneratorPluginSettings::GetMetadataPath()
{
	return GetIntermediatePath() / TEXT("Metadata");
}

FString UHairCardGeneratorPluginSettings::GetGeneratedSettingsPath()
{
	return GetMetadataPath() / TEXT("_generated");
}


FString UHairCardGeneratorPluginSettings::GetTextureImportBaseName() const
{
	return DerivedTextureSettingsName;
}

FString UHairCardGeneratorPluginSettings::GetTextureImportPath() const
{
	return GetIntermediatePath() / TEXT("Output") / DerivedTextureSettingsName;
}

FString UHairCardGeneratorPluginSettings::GetTextureContentPath() const
{
	const FString SettingsFile = GetSettingsFileFromBase(DerivedTextureSettingsName);
	TSharedPtr<FJsonObject> SettingsObj = FJsonHelper::ReadJsonSettings(SettingsFile);
	return FJsonHelper::GetStringFieldDefault(SettingsObj, TEXT("DestinationPath"));
}


TSharedPtr<FJsonObject> UHairCardGeneratorPluginSettings::GetFullParent() const
{
	TSharedPtr<FJsonObject> DerivedFullSettings = TraverseSettingsParents(BaseParentName, 
		[](const TSharedPtr<FJsonObject>& SettingsRoot)
		{
			bool bDerivedGeometry = FJsonHelper::GetBoolFieldDefault(SettingsRoot, TEXT("bReduceCardsFromPreviousLOD"));
			return !bDerivedGeometry;
		}
	);

	return DerivedFullSettings;
}

bool UHairCardGeneratorPluginSettings::HasDerivedTextureSettings() const
{
	// TODO: Cache the settings hierarchy for these checks
	TSharedPtr<FJsonObject> ParentTextureSettings = GetParentTextureSettings();
	return ParentTextureSettings.IsValid();
}

int UHairCardGeneratorPluginSettings::GetDerivedReservedTextureSize() const
{
	TSharedPtr<FJsonObject> ParentTextureSettings = GetParentTextureSettings();
	if ( !ParentTextureSettings.IsValid() )
	{
		return 0;
	}

	return FJsonHelper::GetIntFieldDefault(ParentTextureSettings, TEXT("ReserveTextureSpaceLOD"));
}

FString UHairCardGeneratorPluginSettings::GetDerivedTextureChannelLayout() const
{
	TSharedPtr<FJsonObject> ParentTextureSettings = GetParentTextureSettings();
	if ( !ParentTextureSettings.IsValid() )
	{
		return TEXT("");
	}

	return FJsonHelper::GetStringFieldDefault(ParentTextureSettings, TEXT("ChannelLayout"));
}


FString UHairCardGeneratorPluginSettings::GetSettingsFileFromBase(const FString& SettingsName)
{
	return GetMetadataPath() / SettingsName + TEXT(".json");
}

FString UHairCardGeneratorPluginSettings::GetSettingsFilename() const
{
	return GetSettingsFileFromBase(BaseFilename);
}

FString UHairCardGeneratorPluginSettings::GetGeneratedSettingsFilename() const
{
	return GetGeneratedSettingsPath() / BaseFilename + TEXT(".settings.bin");
}

FString UHairCardGeneratorPluginSettings::GetGeneratedGroupSettingsFilename(int FilterGroupIndex) const
{
	return GetGeneratedSettingsPath() / FilterGroupGenerationSettings[FilterGroupIndex]->GenerateFilename + TEXT(".settings.bin");
}

void UHairCardGeneratorPluginSettings::ResetFromJson()
{
	FString SettingsFile = GetSettingsFilename();
	TSharedPtr<FJsonObject> JsonSettingsObject = FJsonHelper::ReadJsonSettings(SettingsFile);

	// Keep this base filename even if json is different (e.g. copied json without changing base)
	FString TmpBase = BaseFilename;

	// Load settings from json object 
	HairCardSettings_Converter::ResetSettingsFromJson(this, JsonSettingsObject);
	BaseFilename = TmpBase;

	// Load destination path from json
	const FString DestinationPathField = TEXT("DestinationPath");
	if (JsonSettingsObject->HasTypedField<EJson::String>(DestinationPathField))
	{
		DestinationPath.Path = JsonSettingsObject->GetStringField(DestinationPathField);
	}
	
	const FString GroupsArrayFieldName = TEXT("Groups");
	// Load each group settings from json subobjects array
	if ( !JsonSettingsObject->HasTypedField<EJson::Array>(GroupsArrayFieldName) )
	{
		UE_LOGF(LogHairCardGenerator, Warning, "No haircard group settings found in file: %ls", *SettingsFile);
		return;
	}

	const TArray<TSharedPtr<FJsonValue>>& JsonGroups = JsonSettingsObject->GetArrayField(GroupsArrayFieldName);
	for ( int Index = 0; Index < JsonGroups.Num(); ++Index )
	{
		if ( JsonGroups[Index]->Type != EJson::Object )
		{
			UE_LOGF(LogHairCardGenerator, Warning, "Invalid settings at index %d: %ls", Index, *SettingsFile);
			continue;
		}

		// Load subsettings objects from json
		const TSharedPtr<FJsonObject>& GroupSettings = JsonGroups[Index]->AsObject();
		ResetFilterGroupSettings(Index, GroupSettings);
	}
	ResetNumFilterGroups(JsonGroups.Num());
}


void UHairCardGeneratorPluginSettings::WriteToJson()
{
	TSharedRef<FJsonObject> JsonSettingsRoot = MakeShared<FJsonObject>();
	HairCardSettings_Converter::WriteSettingsToJson(this, JsonSettingsRoot);
	// Manually output some necessary settings to json
	JsonSettingsRoot->SetField(TEXT("PreviousLOD"), MakeShared<FJsonValueString>(BaseParentName));
	JsonSettingsRoot->SetField(TEXT("DestinationPath"), MakeShared<FJsonValueString>(DestinationPath.Path));
	JsonSettingsRoot->SetField(TEXT("Version"), MakeShared<FJsonValueString>(Version));

	// Create array of json objects for group settings subobjects
	TArray<TSharedPtr<FJsonValue>> JsonGroups;
	for ( int Index = 0; Index < FilterGroupGenerationSettings.Num(); ++Index )
	{
		TSharedRef<FJsonObject> JsonSettingsGroup = MakeShared<FJsonObject>();
		HairCardSettings_Converter::WriteSettingsToJson(FilterGroupGenerationSettings[Index], JsonSettingsGroup);

		JsonGroups.Add(MakeShared<FJsonValueObject>(JsonSettingsGroup));
	}
	JsonSettingsRoot->SetArrayField(TEXT("Groups"), JsonGroups);

	FString SettingsFile = GetSettingsFilename();

	FString JsonText;
	TSharedRef<TJsonWriter<TCHAR>> JsonWriter = TJsonWriterFactory<TCHAR>::Create(&JsonText);
	if ( !FJsonSerializer::Serialize(JsonSettingsRoot, JsonWriter, true) )
	{
		UE_LOGF(LogHairCardGenerator, Warning, "Unable to write json settings string for object");
		return;
	}

	if ( !FFileHelper::SaveStringToFile(JsonText, *SettingsFile) )
	{
		UE_LOGF(LogHairCardGenerator, Warning, "Unable to write json settings file: %ls", *SettingsFile);
		return;
	}
}

void UHairCardGeneratorPluginSettings::SerializeEditableSettings(FArchive& Ar)
{
	// NOTE: Only append new settings fields at end (FArchive produces reasonable defaults without buffer overflow)
	Ar << DestinationPath.Path;
	Ar << RandomSeed;

	Ar << AtlasSize;
	Ar << DepthMinimum;
	Ar << DepthMaximum;
	Ar << UVSmoothingIterations;
	
	Ar << Version;

	Ar << bReduceCardsFromPreviousLOD;
	Ar << bUseReservedSpaceFromPreviousLOD;
	Ar << ReserveTextureSpaceLOD;
	Ar << bUseGroomAssetStrandWidth;
	Ar << ChannelLayout;
	// TODO: Accurate diff requires conditional for these settings
	Ar << HairWidths;
	Ar << RootScales;
	Ar << TipScales;
}

void UHairCardGeneratorPluginSettings::ResetNumFilterGroups(int Count)
{
	FilterGroupGenerationSettings.SetNum(Count, EAllowShrinking::Yes);
}

void UHairCardGeneratorPluginSettings::MakeSingleFilterGroupSettings()
{
	FilterGroupGenerationSettings.Reset();
	
	for(FName& CardGroup : CardGroupIds)
	{
		FilterGroupGenerationSettings.Add(NewObject<UHairCardGeneratorGroupSettings>(this));

		FilterGroupGenerationSettings.Last()->SetSource(GroomAsset, LODIndex);
		FilterGroupGenerationSettings.Last()->Reset(UHairCardGeneratorGroupSettings::StaticClass()->GetDefaultObject<UHairCardGeneratorGroupSettings>());

		FilterGroupGenerationSettings.Last()->ResetFilterDefault({CardGroup});
		FilterGroupGenerationSettings.Last()->UpdateStrandFilter();
	}

	ResetNumFilterGroups(CardGroupIds.Num());
	PostResetUpdates();
}

void UHairCardGeneratorPluginSettings::BuildFilterGroupSettings(const TArray<TArray<FName>>& FilterCardGroups, const TArray<FName>& FilterGroupNames)
{
	if(FilterGroupNames.Num() == FilterCardGroups.Num())
	{
		FilterGroupGenerationSettings.Reset();
		int32 FilterIndex = 0;
		for(const TArray<FName>& CardGroups : FilterCardGroups)
		{
			TArray<FName> LocalGroups;
			for(const FName& CardGroup : CardGroups)
			{
				if(CardGroupIds.Contains(CardGroup))
				{
					LocalGroups.Add(CardGroup);
				}
			}
			FilterGroupGenerationSettings.Add(NewObject<UHairCardGeneratorGroupSettings>(this));

			FilterGroupGenerationSettings.Last()->SetSource(GroomAsset, LODIndex);
			FilterGroupGenerationSettings.Last()->Reset(UHairCardGeneratorGroupSettings::StaticClass()->GetDefaultObject<UHairCardGeneratorGroupSettings>());

			if (!LocalGroups.IsEmpty())
			{
				FilterGroupGenerationSettings.Last()->ResetFilterDefault(LocalGroups);
			}
			else
			{
				FilterGroupGenerationSettings.Last()->ResetFilterDefault(CardGroupIds);
			}
			FilterGroupGenerationSettings.Last()->UpdateStrandFilter();
			FilterGroupGenerationSettings.Last()->SetFilterName(FilterGroupNames[FilterIndex]);
			++FilterIndex;
		}

		ResetNumFilterGroups(FilterGroupGenerationSettings.Num());
		PostResetUpdates();
	}
}

void UHairCardGeneratorPluginSettings::ResetFilterGroupSettings(int FilterGroupIndex, TObjectPtr<UHairCardGeneratorGroupSettings> OptionsTemplate)
{
	if ( FilterGroupGenerationSettings.Num() <= FilterGroupIndex )
		FilterGroupGenerationSettings.Add(NewObject<UHairCardGeneratorGroupSettings>(this));

	FilterGroupGenerationSettings[FilterGroupIndex]->SetSource(GroomAsset, LODIndex);
    FilterGroupGenerationSettings[FilterGroupIndex]->Reset(OptionsTemplate);

	FilterGroupGenerationSettings[FilterGroupIndex]->ResetFilterDefault(CardGroupIds);
	FilterGroupGenerationSettings[FilterGroupIndex]->UpdateStrandFilter();

	FilterGroupGenerationSettings[FilterGroupIndex]->UpdateGenerateFilename(BaseFilename, FilterGroupIndex);
}

void UHairCardGeneratorPluginSettings::ResetFilterGroupSettings(int FilterGroupIndex, const TSharedPtr<FJsonObject>& OptionsTemplate)
{
	if ( FilterGroupGenerationSettings.Num() <= FilterGroupIndex )
		FilterGroupGenerationSettings.Add(NewObject<UHairCardGeneratorGroupSettings>(this));

	FilterGroupGenerationSettings[FilterGroupIndex]->SetSource(GroomAsset, LODIndex);
    FilterGroupGenerationSettings[FilterGroupIndex]->Reset(OptionsTemplate);

	FilterGroupGenerationSettings[FilterGroupIndex]->ResetFilterDefault(CardGroupIds);
	FilterGroupGenerationSettings[FilterGroupIndex]->UpdateStrandFilter();

	FilterGroupGenerationSettings[FilterGroupIndex]->UpdateGenerateFilename(BaseFilename, FilterGroupIndex);
}

bool UHairCardGeneratorPluginSettings::ValidateStrandFilters(TMap<uint32,uint32>& ErrorCountMap)
{
	bool NeedsUpdate = false;
	for ( TObjectPtr<UHairCardGeneratorGroupSettings> FilterGroupSettings : FilterGroupGenerationSettings )
	{
		if ( FilterGroupSettings->StrandFilterChecksDirty )
		{
			NeedsUpdate = true;
			break;
		}
	}

	if ( NeedsUpdate )
	{
		ValidateStrandFiltersInternal();
	}

	for ( TObjectPtr<UHairCardGeneratorGroupSettings> FilterGroupSettings : FilterGroupGenerationSettings )
	{
		FilterGroupSettings->StrandFilterChecksDirty = false;
	}

	ErrorCountMap = StrandErrorCountMap;
	return (UnassignedStrandsCount > 0);
}

void UHairCardGeneratorPluginSettings::UpdateStrandFilterAssignment()
{
	FHairDescription HairDesc = GroomAsset->GetHairDescription();
	StrandFilterGroupIndexMap.Reset(HairDesc.GetNumStrands());

	for ( const FHairStrandAttribsRefProxy& StrandAttribs : FStrandAttributesRange(HairDesc, FHairStrandAttribsConstIterator::DefaultIgnore) )
	{
		StrandFilterGroupIndexMap.Add(AssignStrandToFilterGroup(StrandAttribs));
	}

	StrandFilterGroupIndexMap.Shrink();
}

void UHairCardGeneratorPluginSettings::ValidateStrandFiltersInternal()
{
	UnassignedStrandsCount = 0;
	StrandErrorCountMap.Reset();

	FHairDescription HairDesc = GroomAsset->GetHairDescription();
	for ( const FHairStrandAttribsRefProxy& StrandAttribs : FStrandAttributesRange(HairDesc, FHairStrandAttribsConstIterator::DefaultIgnore) )
	{
		uint32 AssignmentBits = CheckAllStrandAssignments(StrandAttribs);
		// Check for a single 1 (power of 2)
		bool AssignedMultipleGroups = (AssignmentBits & (AssignmentBits - 1)) != 0;
		if ( AssignmentBits == 0 )
			UnassignedStrandsCount += 1;
		else if ( AssignedMultipleGroups )
			StrandErrorCountMap.FindOrAdd(AssignmentBits, 0) += 1;
	}
}

int32 UHairCardGeneratorPluginSettings::AssignStrandToFilterGroup(const FHairStrandAttribsRefProxy& StrandAttribs)
{
	for ( int Index = 0; Index < FilterGroupGenerationSettings.Num(); ++Index )
	{
		bool InGroup = FilterGroupGenerationSettings[Index]->StrandsFilter->FilterStrand(StrandAttribs);
		if ( InGroup )
			return Index;
	}

	return INDEX_NONE;
}

uint32 UHairCardGeneratorPluginSettings::CheckAllStrandAssignments(const FHairStrandAttribsRefProxy& StrandAttribs)
{
	uint32 Assignment = 0;
	for ( int Index = 0; Index < FilterGroupGenerationSettings.Num(); ++Index )
	{
		if ( FilterGroupGenerationSettings[Index]->StrandsFilter->FilterStrand(StrandAttribs) )
			Assignment |= (1 << Index);
	}

	return Assignment;
}

bool UHairCardGeneratorPluginSettings::CanEditChange(const FProperty* InProperty) const
{
	bool bCanEdit = Super::CanEditChange(InProperty);
	if (!bCanEdit)
	{
		return false;
	}

	FName PropertyName = InProperty->GetFName();

	// Disable most settings if Reducing from previous LOD
	if ( bReduceCardsFromPreviousLOD )
	{
		return !((PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, bGenerateGeometryForAllGroups))
			|| (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, GenerateForGroomGroup))
			|| (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, RandomSeed))
			|| (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, AtlasSize))
			|| (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, bUseReservedSpaceFromPreviousLOD))
			|| (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, ReserveTextureSpaceLOD))
			|| (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, bUseGroomAssetStrandWidth)));
	}
	
	if ( (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, AtlasSize)) )
	{
		return !bUseReservedSpaceFromPreviousLOD;
	}

	if ( (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, bGenerateGeometryForAllGroups)) )
	{
		return (GenerateForGroomGroup == 0);
	}

	return true;
}

void UHairCardGeneratorPluginSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, BaseFilename))
	{
		if (BaseFilename.IsEmpty())
		{
			BaseFilename = HairCardSettings_Helpers::GetDefaultBaseNameForHairCards(GroomAsset, LODIndex, FString());
		}
		else
		{
			// We use this name to generate paths and import object names so need it to be correctly sanitized (and replace "\", "/" characters)
			BaseFilename = ObjectTools::SanitizeInvalidChars(BaseFilename, INVALID_OBJECTNAME_CHARACTERS TEXT("\\"));
		}

		if ( !ResetFromSettingsJson(FString(), FString()) )
		{
			UpdateOutputPaths();
		}

		// NOTE: This must be updated if basename or derived lod options change
		UpdateParentInfo();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, DestinationPath))
	{
		if (DestinationPath.Path.IsEmpty())
		{
			DestinationPath.Path = HairCardSettings_Helpers::GetDefaultDestinationPathForHairCards(GroomAsset, LODIndex, FString(), FString());
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, bReduceCardsFromPreviousLOD) 
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, bUseReservedSpaceFromPreviousLOD))
	{
		UpdateParentInfo();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorPluginSettings, bUseGroomAssetStrandWidth))
	{
		UpdateHairWidths();
	}
}

void UHairCardGeneratorPluginSettings::BuildDDCKey(FArchive& Ar)
{
	SerializeEditableSettings(Ar);

	int FilterGroupSettingsCount = FilterGroupGenerationSettings.Num();
	Ar << FilterGroupSettingsCount;

	for ( int i=0; i < FilterGroupGenerationSettings.Num(); ++i )
		FilterGroupGenerationSettings[i]->BuildDDCSubKey(Ar);
}


bool UHairCardGeneratorPluginSettings::IsCompatibleSettings(UHairCardGenerationSettings* SettingsObject)
{
	return SettingsObject && SettingsObject->IsA<UHairCardGeneratorPluginSettings>();
}

FString UHairCardGeneratorPluginSettings::GetGroomName() const
{
	return GroomAsset->GetName();
}

template <typename T>
void SetIfValid(T* OutPtr, const T& Val)
{
	if (OutPtr)
	{
		*OutPtr = Val;
	}
}

bool UHairCardGeneratorPluginSettings::CanReduceFromLOD(FText* OutInvalidInfo) const
{
    if ( GetLODIndex() < 1 )
    {
		SetIfValid(OutInvalidInfo, LOCTEXT("ReduceFromLOD.LOD0.ToolTip", "Cannot reduce LOD 0 (must run full generation)"));
        return false;
    }

    if ( !ValidChannelLayouts() )
    {
        SetIfValid(OutInvalidInfo, LOCTEXT("ReduceFromLOD.InconsistentGroupLayouts.ToolTip", "Inconsistent texture layouts for groom groups at this LOD"));
        return false;
    }
    
    TSharedPtr<FJsonObject> ParentSettingsJson = GetFullParent();
    if ( !ParentSettingsJson.IsValid() )
    {
        SetIfValid(OutInvalidInfo, LOCTEXT("ReduceFromLOD.InvalidParent.ToolTip", "All lower LODs must be generated using the hair card generator tool"));
        return false;
    }

    if ( !ParentSettingsJson->HasTypedField<EJson::String>(TEXT("ChannelLayout")) )
    {
        SetIfValid(OutInvalidInfo, LOCTEXT("ReduceFromLOD.InvalidChannelLayout.ToolTip", "Invalid texture layout setting in previous LOD"));
        return false;
    }

    UEnum* EnumClass = StaticEnum<EHairTextureLayout>();
    if ( !EnumClass || ParentSettingsJson->GetStringField(TEXT("ChannelLayout")) != EnumClass->GetNameStringByValue((int64)GetChannelLayout()) )
    {
		SetIfValid(OutInvalidInfo, LOCTEXT("ReduceFromLOD.InconsistentParentLayout.ToolTip", "Parent texture layout setting differs from current LOD texture layout"));
        return false;
    }

    return true;
}

bool UHairCardGeneratorPluginSettings::CanUseReservedTx(FText* OutInvalidInfo) const
{
    if ( bReduceCardsFromPreviousLOD )
    {
        SetIfValid(OutInvalidInfo, LOCTEXT("UseReservedTx.Reducing.ToolTip", "Reduced card geometry will use previous LOD texture UVs"));
        return false;
    }

    if ( !HasDerivedTextureSettings() )
    {
        SetIfValid(OutInvalidInfo, LOCTEXT("UseReservedTx.InvalidParent.ToolTip", "Lower LODs must be generated using hair card generator tool with reserved space"));
        return false;
    }

    // TODO: Handle limiting/reserving texture by using texture resolution to compute reserved space in pixels
    if ( GetDerivedReservedTextureSize() < 5 )
    {
        SetIfValid(OutInvalidInfo, LOCTEXT("UseReservedTx.NoReservedSpace.ToolTip", "No space reserved in parent LOD chain"));
        return false;
    }

    UEnum* EnumClass = StaticEnum<EHairTextureLayout>();
    if ( !EnumClass || GetDerivedTextureChannelLayout() != EnumClass->GetNameStringByValue((int64)GetChannelLayout()) )
    {
        SetIfValid(OutInvalidInfo, LOCTEXT("UseReservedTx.InconsistentReservedLayout.ToolTip", "Parent reserved texture layout setting differs from current LOD texture layout"));
        return false;
    }
    
    return true;
}

/* UHairCardGeneratorGroupSettings
 *****************************************************************************/

void UHairCardGeneratorGroupSettings::Reset(TObjectPtr<UHairCardGeneratorGroupSettings> TemplateObject)
{
	HairCardSettings_Helpers::ResetSettingsFromTemplate(this, TemplateObject);
}

void UHairCardGeneratorGroupSettings::Reset(const TSharedPtr<FJsonObject>& TemplateSettings)
{
	HairCardSettings_Converter::ResetSettingsFromJson(this, TemplateSettings);
}

void UHairCardGeneratorGroupSettings::ResetFilterDefault(const TArray<FName>& CardGroupNames)
{
	// HACK: Need to use a demonstrably invalid value by default to check against so we can only update from "default"
	if ( !ApplyToCardGroups.Contains(NAME_None) )
		return;

	// Default filter includes all group ids
	ApplyToCardGroups.Reset();
	for ( FName id : CardGroupNames )
		ApplyToCardGroups.Add(id);

	// ApplyToCardGroups.Sort([](FName a, FName b){return a < b;});
}

void UHairCardGeneratorGroupSettings::UpdateStrandFilter()
{
	StrandsFilter = MakeUnique<FStrandFilterOp_InCardGroup>(ApplyToCardGroups);

	StrandFilterChecksDirty = true;
	SetStrandStats();
}

void UHairCardGeneratorGroupSettings::UpdateGenerateFilename(const FString& BaseFilename, int index)
{
	GenerateFilename = BaseFilename + TEXT("_GIDX") + FString::FromInt(index);
}

void UHairCardGeneratorGroupSettings::UpdateParentName(const FString& BaseParentName, int index)
{
	ParentName = TEXT("");
	if ( !BaseParentName.IsEmpty() )
	{
		ParentName = BaseParentName + TEXT("_GIDX") + FString::FromInt(index);
	}
}

void UHairCardGeneratorGroupSettings::RemoveSettingsFilterGroup()
{
	UHairCardGeneratorPluginSettings* PluginSettings = Cast<UHairCardGeneratorPluginSettings>(GetOuter());
	PluginSettings->RemoveSettingsFilterGroup(this);
}

bool UHairCardGeneratorGroupSettings::CanEditChange(const FProperty* InProperty) const
{
	bool bCanEdit = Super::CanEditChange(InProperty);
	if (!bCanEdit)
	{
		return false;
	}

	FName PropertyName = InProperty->GetFName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorGroupSettings, MaxVerticalSegmentsPerCard))
	{
		return !UseAdaptiveSubdivision;
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorGroupSettings, TargetTriangleCount))
	{
		return UseAdaptiveSubdivision;
	}

	return true;
}

void UHairCardGeneratorGroupSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UHairCardGeneratorGroupSettings, ApplyToCardGroups))
	{
		// ApplyToCardGroups.Sort([](FName a, FName b){return a < b;});
		UpdateStrandFilter();
	}
}

void UHairCardGeneratorGroupSettings::BuildDDCSubKey(FArchive& Ar)
{
	SerializeGroupSettings(Ar);
}

void UHairCardGeneratorGroupSettings::SetSource(TObjectPtr<const UGroomAsset> SourceGroom, int32 GenLODIndex)
{
	GroomAsset = SourceGroom;
	LODIndex = GenLODIndex;
}

uint8 UHairCardGeneratorGroupSettings::GetFilterGroupGeneratedFlags() const
{
	// TODO: Cache these checks for UI and update on posteditchange
	UHairCardGeneratorPluginSettings* PluginSettings = Cast<UHairCardGeneratorPluginSettings>(GetOuter());
	uint8 PipelineFlags = PluginSettings->GetPipelineGeneratedDifferences();
	if (PipelineFlags == (uint8)EHairCardGenerationSettingsCategories::All)
		return PipelineFlags;

	return PipelineFlags | GetPipelineFilterGroupDifferences();
}

bool UHairCardGeneratorGroupSettings::GetReduceFromPreviousLOD() const
{
	UHairCardGeneratorPluginSettings* PluginSettings = Cast<UHairCardGeneratorPluginSettings>(GetOuter());
	return PluginSettings->bReduceCardsFromPreviousLOD;
}

FString UHairCardGeneratorGroupSettings::GetGeneratedFilterGroupSettingsFilename() const
{
	return UHairCardGeneratorPluginSettings::GetGeneratedSettingsPath() / GenerateFilename + TEXT(".settings.bin");
}

void UHairCardGeneratorGroupSettings::SerializeGroupSettings(FArchive& Ar)
{
	// NOTE: Only append new settings fields at end (FArchive produces reasonable defaults without buffer overflow)
	Ar << ApplyToCardGroups;
	Ar << TargetNumberOfCards;
	Ar << MaxNumberOfFlyaways;
	Ar << UseMultiCardClumps;

	Ar << UseAdaptiveSubdivision;
	Ar << MaxVerticalSegmentsPerCard;
	Ar << TargetTriangleCount;

	Ar << NumberOfTexturesInAtlas;
	Ar << StrandWidthScalingFactor;
	Ar << UseOptimizedCompressionFactor;
}

uint8 UHairCardGeneratorGroupSettings::GetPipelineFilterGroupDifferences() const
{
	const FString GroupSettingsBinFile = GetGeneratedFilterGroupSettingsFilename();
	if ( !FPaths::FileExists(GroupSettingsBinFile) )
		return (uint8)EHairCardGenerationPipeline::All;

	TArray<uint8> SettingsBytes;
	if ( !FFileHelper::LoadFileToArray(SettingsBytes, *GroupSettingsBinFile, 0) )
		return (uint8)EHairCardGenerationPipeline::All;

	FMemoryReader MemReader(SettingsBytes, true);

	TObjectPtr<UHairCardGeneratorGroupSettings> DiffObject = NewObject<UHairCardGeneratorGroupSettings>();
	DiffObject->SerializeGroupSettings(MemReader);

	return HairCardSettings_Helpers::PipelineSettingsDiff(this, DiffObject);
}

void UHairCardGeneratorGroupSettings::ClearPipelineGeneratedFilterGroup() const
{
	const FString GroupSettingsBinFile = GetGeneratedFilterGroupSettingsFilename();

	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
	FileManager.DeleteFile(*GroupSettingsBinFile);
}

void UHairCardGeneratorGroupSettings::WritePipelineGeneratedFilterGroup()
{
	const FString GeneratedPath = UHairCardGeneratorPluginSettings::GetGeneratedSettingsPath();
	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
	if ( !FileManager.DirectoryExists(*GeneratedPath) )
		FileManager.CreateDirectoryTree(*GeneratedPath);

	TArray<uint8> SettingsBytes;
	SettingsBytes.Reserve(64);
	FMemoryWriter MemWriter(SettingsBytes, true);

	SerializeGroupSettings(MemWriter);

	const FString GroupSettingsBinFile = GetGeneratedFilterGroupSettingsFilename();
	FFileHelper::SaveArrayToFile(SettingsBytes, *GroupSettingsBinFile);
}

void UHairCardGeneratorGroupSettings::SetStrandStats()
{
	if ( !GroomAsset )
		return;

	float Length = 0.0;
	float AvgCurvRadius = 0.0;
	int CountFiltered = 0;

	FHairDescription HairDesc = GroomAsset->GetHairDescription();
	for ( const FHairStrandAttribsRefProxy& StrandAttribs : FStrandAttributesRange(HairDesc, FHairStrandAttribsConstIterator::DefaultIgnore) )
	{
		if ( StrandsFilter->FilterStrand(StrandAttribs) )
		{
			CountFiltered ++;

			MatrixXf VertexPositions(StrandAttribs.NumVerts(), 3);

			for (int StrandVertIndex = 0; StrandVertIndex < StrandAttribs.NumVerts(); ++StrandVertIndex)
			{
				const FVector3f& VertexPosition = StrandAttribs.GetVertexPosition(StrandVertIndex);

				for (int i=0; i<3; i++)
				{
					VertexPositions(StrandVertIndex, i) = VertexPosition[i];
				}
			}

			Length += FHairCardGenCardSubdivider::GetCurveLength(VertexPositions);
			AvgCurvRadius += FHairCardGenCardSubdivider::GetAverageCurvatureRadius(VertexPositions);
		}
	}

	StrandCount = CountFiltered;
}

#undef LOCTEXT_NAMESPACE
