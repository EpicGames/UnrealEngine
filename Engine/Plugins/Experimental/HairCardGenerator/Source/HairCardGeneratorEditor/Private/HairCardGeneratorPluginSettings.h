// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroomAssetCards.h" // for UHairCardGenerationSettings
#include "HairCardGenStrandFilter.h"

#include "HairCardGeneratorPluginSettings.generated.h"

class FJsonObject;

namespace HairCardSettings_Helpers
{
	void GetStrandCardGroupInfo(TObjectPtr<const UGroomAsset> GroomAsset, TArray<FName>& OutGrpIds, TArray<int32>& OutGrpCount);
};

UENUM(BlueprintType)
enum class EHairCardAtlasSize : uint8
{
	Uninitialized = 0  UMETA(Hidden),
	AtlasSize1024 = 10 UMETA(DisplayName = "1024 x 1024"),
	AtlasSize2048 = 11 UMETA(DisplayName = "2048 x 2048"),
	AtlasSize4096 = 12 UMETA(DisplayName = "4096 x 4096"),
	AtlasSize8192 = 13 UMETA(DisplayName = "8192 x 8192")
};

// Helper enumeration for settings dirty bits for pipeline steps
UENUM()
enum class EHairCardGenerationPipeline : uint8
{
	None = 0x0,
	// Pipeline Steps
	StrandClustering = 0x01,   // 1. Cluster strands into clumps for creating cards
	GeometryGeneration = 0x02, // 2. Geometry generation from strand clusters (clumps)
	TextureClustering = 0x04,  // 3. Cluster clumps into num_texures, and find the clump that's closest to cluster mean
	TextureLayout = 0x08,      // 4. Layout representative clumps in texture atlas
	TextureRendering = 0x10,   // 5. Render representative clumps to texture atlas
	GenerateMesh = 0x20,       // 6. Generate a mesh from all clumps (card) geometry
	ImportTextures = 0x40,     // 7. Import textures and assign to card asset slots

	// HACK: These flag combos should probably be in a separate enum
	// Useful shorthand for flag update checks
	// Some update required
	All = StrandClustering
			| GeometryGeneration
			| TextureClustering
			| TextureLayout
			| TextureRendering
			| GenerateMesh
			| ImportTextures,
	// Group pipeline stage update required
	GroupUpdate = StrandClustering
				| GeometryGeneration
				| TextureClustering,
	// Textures will be updated
	TextureUpdate = TextureLayout
				| TextureRendering,
	// Asset import/mesh regen required
	ImportUpdate = GenerateMesh
				| ImportTextures
};

ENUM_CLASS_FLAGS(EHairCardGenerationPipeline)

UENUM()
enum class EHairCardGenerationSettingsCategories : uint8
{
	// Settings Categories
	None = 0x0,
	All = (uint8)EHairCardGenerationPipeline::StrandClustering
				| (uint8)EHairCardGenerationPipeline::GeometryGeneration
				| (uint8)EHairCardGenerationPipeline::TextureClustering
				| (uint8)EHairCardGenerationPipeline::TextureLayout
				| (uint8)EHairCardGenerationPipeline::TextureRendering
				| (uint8)EHairCardGenerationPipeline::GenerateMesh
				| (uint8)EHairCardGenerationPipeline::ImportTextures,
	Asset = All,
	LevelOfDetail = All,
	Randomness = All,
	Cards = All,
	Geometry = (uint8)EHairCardGenerationPipeline::GeometryGeneration
				| (uint8)EHairCardGenerationPipeline::TextureClustering
				| (uint8)EHairCardGenerationPipeline::TextureLayout
				| (uint8)EHairCardGenerationPipeline::TextureRendering
				| (uint8)EHairCardGenerationPipeline::GenerateMesh
				| (uint8)EHairCardGenerationPipeline::ImportTextures,
	Textures = (uint8)EHairCardGenerationPipeline::TextureClustering
				| (uint8)EHairCardGenerationPipeline::TextureLayout
				| (uint8)EHairCardGenerationPipeline::TextureRendering
				| (uint8)EHairCardGenerationPipeline::GenerateMesh
				| (uint8)EHairCardGenerationPipeline::ImportTextures,
	TextureRendering = (uint8)EHairCardGenerationPipeline::TextureLayout
				| (uint8)EHairCardGenerationPipeline::TextureRendering
				| (uint8)EHairCardGenerationPipeline::GenerateMesh
				| (uint8)EHairCardGenerationPipeline::ImportTextures,
	Import = (uint8)EHairCardGenerationPipeline::GenerateMesh
				| (uint8)EHairCardGenerationPipeline::ImportTextures,
};

namespace HairCardSettings_Helpers
{
	HAIRCARDGENERATOREDITOR_API FString GetGroomDirectoryForHairCards(TObjectPtr<const UGroomAsset> GroomAsset);
	HAIRCARDGENERATOREDITOR_API FString GetGroomNameForHairCards(TObjectPtr<const UGroomAsset> GroomAsset);
	HAIRCARDGENERATOREDITOR_API FString GetDefaultBaseNameForHairCards(TObjectPtr<const UGroomAsset> GroomAsset, int LODIndex, const FString& DefaultName);
	HAIRCARDGENERATOREDITOR_API FString GetDefaultDestinationPathForHairCards(TObjectPtr<const UGroomAsset> GroomAsset, int LODIndex, 
						const FString& DefaultDirectory, const FString& DefaultName);
}

/**
 * Self contained hair card generation settings. Separated/Duplicated from the 
 * groom asset to: (1) constrain the prompt window's details view, and (2) let
 * the user modify it without mutating the original groom (settings should only 
 * be applied/saved when the process is done).
 * 
 * UHairCardGenerationPluginSettings holds all the generation settings required
 * to regenerate hair cards for a particular groom asset. Implements the
 * UHairCardGenerationSettings interface so that it can be used with the
 * hair card generation extension module (see IHairCardGenerator API).
*/
UCLASS(BlueprintType, config=Editor)
class HAIRCARDGENERATOREDITOR_API UHairCardGeneratorPluginSettings : public UHairCardGenerationSettings
{
	GENERATED_BODY()

public:
	// Destination content path for generated mesh/texture assets
	UPROPERTY(EditAnywhere, Category="Import", meta=(ContentDir))
	FDirectoryPath DestinationPath;

	// Base filename of generated mesh/texture assets
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Asset")
	FString BaseFilename;

	// Use previous LOD generated cards and textures but reduce triangle count and flyaways
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Level Of Detail")
	bool bReduceCardsFromPreviousLOD = false;

	// Generate geometry for all groom groups on group 0
	UPROPERTY(Transient, EditAnywhere, BlueprintReadonly, Category="Asset", meta=(AllowPrivateAccess))
	bool bGenerateGeometryForAllGroups = true;

	// Seed value for pseudo-random number generation (set to a specific value for repeatable results)
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Randomness", meta=(ClampMin="0", ClampMax="10000"))
	int RandomSeed = 0;

	// Place new card textures in reserved space from previous LOD
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Texture Rendering")
	bool bUseReservedSpaceFromPreviousLOD = false;

	// Size of hair card texture atlases
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Texture Rendering")
	EHairCardAtlasSize AtlasSize = EHairCardAtlasSize::AtlasSize4096;

	// Texture layout selected (NOTE: This automatically pulls from group 0)
	UPROPERTY(BlueprintReadOnly, Category="Texture Rendering")
	EHairTextureLayout ChannelLayout = EHairTextureLayout::Layout0;

	// Percentage of texture atlas space to reserve for higher LODs 
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Texture Rendering", meta=(ClampMin="0", ClampMax="75"))
	int ReserveTextureSpaceLOD = 0;

	// Use strand width
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Texture Rendering")
	bool bUseGroomAssetStrandWidth = true;

	UPROPERTY(BlueprintReadOnly, Category="Texture Rendering")
	bool bOverrideHairWidth;

	UPROPERTY(BlueprintReadOnly, Category="Texture Rendering")
	TArray<float> HairWidths;

	UPROPERTY(BlueprintReadOnly, Category="Texture Rendering")
	TArray<float> RootScales;

	UPROPERTY(BlueprintReadOnly, Category="Texture Rendering")
	TArray<float> TipScales;

	// Minimum strand depth value (mapped to 0 in depth texture)
	UPROPERTY(BlueprintReadOnly, Category="Texture Rendering|Advanced", meta=(ClampMin="-10.0", ClampMax="0.0"))
	float DepthMinimum = -2.0f;

	// Maximum strand depth value (mapped to 1 in depth texture)
	UPROPERTY(BlueprintReadOnly, Category="Texture Rendering|Advanced", meta=(ClampMin="0.0", ClampMax="10.0"))
	float DepthMaximum = 2.0f;

	// Number of iterations for smoothing strand UVs
	UPROPERTY(BlueprintReadOnly, Category="Texture Rendering|Advanced", meta=(ClampMin="0", ClampMax="50"))
	int32 UVSmoothingIterations = 25;
	
	UPROPERTY(Transient, BlueprintReadOnly, Category="Asset")
	FString OutputPath;

	void SetSource(TObjectPtr<UGroomAsset> SourceObject, int32 GenLODIndex = 0, int32 PhysGroupIndex = 0);
	// Load last-run values for options from card source description or defaults if no last-run info
	void ResetFromSourceDescription(const FHairGroupsCardsSourceDescription& SourceDesc, const FString& DefaultDirectory, const FString& DefaultName);
	// Load settings from Json file if one exists
	bool ResetFromSettingsJson(const FString& DefaultDirectory, const FString& DefaultName);
	// Load default values for all options
	void ResetToDefault(const FString& DefaultDirectory, const FString& DefaultName);
	void ResetFilterGroupSettingsToDefault(int FilterGroupIndex);

	// UI Hooks for adding/removing settings groups
	void AddNewSettingsFilterGroup();
	void RemoveSettingsFilterGroup(TObjectPtr<UHairCardGeneratorGroupSettings> SettingsGroup);

	/** Make filter settings for each groups */
	void MakeSingleFilterGroupSettings();

	/** Build the filter group settings according to a list of valid card groups and names */
	void BuildFilterGroupSettings(const TArray<TArray<FName>>& FilterCardGroups, const TArray<FName>& FilterGroupNames);

	// Output settings to json file
	void WriteGenerationSettings();

	// Handle mapping settings differences to regeneration requirements
	static bool CheckGenerationFlags(uint8 NeedsGenFlags, const EHairCardGenerationPipeline PipelineStage);
	uint8 GetAllPipelineGeneratedDifferences() const;
	uint8 GetPipelineGeneratedDifferences() const;
	void ClearPipelineGenerated() const;
	void WritePipelineGenerated();
	uint8 GetPipelineFilterGroupDifferences(int FilterGroupIndex) const;
	void ClearPipelineGeneratedFilterGroup(int FilterGroupIndex) const;
	void WritePipelineGeneratedFilterGroup(int FilterGroupIndex);

	// Handle checking and applying group settings strand filters
	bool ValidateStrandFilters(TMap<uint32,uint32>& ErrorCountMap);
	void UpdateStrandFilterAssignment();

	//~ Begin UObject interface
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface

	//~ Begin UHairCardGenerationSettings interface
	virtual void BuildDDCKey(FArchive& Ar) override;
	//~ End UHairCardGenerationSettings interface

	static bool IsCompatibleSettings(UHairCardGenerationSettings* SettingsObject);

	static FString GetIntermediatePath();
	static FString GetMetadataPath();
	static FString GetGeneratedSettingsPath();

	FString GetTextureImportBaseName() const;
	FString GetTextureImportPath() const;
	FString GetTextureContentPath() const;

	TSharedPtr<FJsonObject> GetFullParent() const;
	bool HasDerivedTextureSettings() const;
	int GetDerivedReservedTextureSize() const;
	FString GetDerivedTextureChannelLayout() const;
	
	FString GetGroomName() const;

	UGroomAsset* GetGroomAsset() const {return GroomAsset;}
	int32 GetLODIndex() const { return LODIndex; }
	int32 GetGenerateForGroomGroup() const { return GenerateForGroomGroup; }
	void SetGenerateForGroomGroup(int GroupIndex) { GenerateForGroomGroup = GroupIndex; }
	EHairTextureLayout GetChannelLayout() const { return ChannelLayout; }
	bool ValidChannelLayouts() const { return bValidChannelLayouts; }

	bool CanReduceFromLOD(FText* OutInvalidInfo = nullptr) const;
	bool CanUseReservedTx(FText* OutInvalidInfo = nullptr) const;

	bool GetForceRegenerate() const {return bForceRegen;}
	void SetForceRegenerate(bool bForceRegenerate){bForceRegen = bForceRegenerate;}

	TArray<TObjectPtr<UHairCardGeneratorGroupSettings>>& GetFilterGroupSettings() { return FilterGroupGenerationSettings; }
	const TArray<TObjectPtr<UHairCardGeneratorGroupSettings>>& GetFilterGroupSettings() const { return FilterGroupGenerationSettings; }

	TObjectPtr<UHairCardGeneratorGroupSettings>& GetFilterGroupSettings(int Index) { return FilterGroupGenerationSettings[Index]; }
	const TObjectPtr<UHairCardGeneratorGroupSettings>& GetFilterGroupSettings(int Index) const { return FilterGroupGenerationSettings[Index]; }

	void PostResetUpdates();

private:
	void UpdateOutputPaths();

	void UpdateChannelLayout();

	void UpdateParentInfo();
	void UpdateHairWidths();

	void EnforceValidLODSettings();

	bool FindDerivedTextureSettings();
	TSharedPtr<FJsonObject> GetParentTextureSettings() const;

	static TSharedPtr<FJsonObject> TraverseSettingsParents(const FString& SettingsName, TFunction<bool(const TSharedPtr<FJsonObject>&)> StopCond);

	static FString GetSettingsFileFromBase(const FString& SettingsName);
	FString GetSettingsFilename() const;
	FString GetGeneratedSettingsFilename() const;
	FString GetGeneratedGroupSettingsFilename(int GroupIndex) const;
	void ResetFromJson();
	void WriteToJson();

	void SerializeEditableSettings(FArchive& Ar);

	void ResetNumFilterGroups(int Count);
	void ResetFilterGroupSettings(int FilterGroupIndex, TObjectPtr<UHairCardGeneratorGroupSettings> OptionsTemplate);
	void ResetFilterGroupSettings(int FilterGroupIndex, const TSharedPtr<FJsonObject>& OptionsTemplate);

	void ValidateStrandFiltersInternal();
	int32 AssignStrandToFilterGroup(const FHairStrandAttribsRefProxy& StrandAttribs);
	uint32 CheckAllStrandAssignments(const FHairStrandAttribsRefProxy& StrandAttribs);

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Asset", meta=(AllowPrivateAccess))
	int32 LODIndex = INDEX_NONE;

	// Static HCS version number to force regeneration on tool updates
	UPROPERTY(BlueprintReadOnly, Category="Asset", meta=(AllowPrivateAccess))
	FString Version = TEXT("0.4");

	UPROPERTY(BlueprintReadOnly, Category="Settings Groups", meta=(AllowPrivateAccess))
	TArray<TObjectPtr<UHairCardGeneratorGroupSettings>> FilterGroupGenerationSettings;

	// The groom group for which geometry will be generated
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadonly, Category="Asset", meta=(AllowPrivateAccess))
	int32 GenerateForGroomGroup = INDEX_NONE;

	UPROPERTY(Transient, BlueprintReadOnly, Category="Asset", meta=(AllowPrivateAccess))
	FString IntermediatePath;

	// The base name of direct parent LOD level, used for resubdividing geometry or writing to reserved texture layout regions
	UPROPERTY(Transient, BlueprintReadOnly, Category="Level Of Detail", meta=(AllowPrivateAccess))
	FString BaseParentName;

	// The base name of LOD from which texture info/outputs will be pulled (for writing to reserved texture layouts)
	UPROPERTY(Transient, BlueprintReadOnly, Category="Level Of Detail", meta=(AllowPrivateAccess))
	FString DerivedTextureSettingsName;

	UPROPERTY(Transient, BlueprintReadOnly, Category="Settings Groups", meta=(AllowPrivateAccess))
	TArray<FName> CardGroupIds;

	UPROPERTY(Transient,BlueprintReadOnly, Category="Settings Groups", meta=(AllowPrivateAccess))
	TArray<int> StrandFilterGroupIndexMap;

	// All group texture layouts are the same (required for current texture generation)
	UPROPERTY(Transient)
	bool bValidChannelLayouts;

	UPROPERTY(Transient)
	TObjectPtr<UGroomAsset> GroomAsset = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<const UStaticMesh> OldGeneratedMesh = nullptr;

	bool bForceRegen;
	uint32 UnassignedStrandsCount;
	TMap<uint32,uint32> StrandErrorCountMap;
};


/** 
 * Actual generation settings. Held in TArray, with strand filters applied to determine
 * which settings apply to each strand (See FComposableStrandFilter).
 */
UCLASS(BlueprintType, config=Editor)
class UHairCardGeneratorGroupSettings : public UObject
{
	GENERATED_BODY()

	friend class UHairCardGeneratorPluginSettings;
private:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Settings Group", meta=(AllowPrivateAccess))
	int32 StrandCount = 0;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Asset", meta=(AllowPrivateAccess))
	FString GenerateFilename;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Asset", meta=(AllowPrivateAccess))
	int32 LODIndex = INDEX_NONE;

	UPROPERTY(Transient, BlueprintReadOnly, Category="Level Of Detail", meta=(AllowPrivateAccess))
	FString ParentName;

	UPROPERTY(Transient)
	TObjectPtr<const UGroomAsset> GroomAsset = nullptr;

	bool StrandFilterChecksDirty = false;
	TUniquePtr<FComposableStrandFilterOp> StrandsFilter;

	UPROPERTY(EditAnywhere, Category="Cards")
	FName FilterName;
	
public:
	// Specify which strands (card group ids) this settings group applies to
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Cards")
	TSet<FName> ApplyToCardGroups = {NAME_None};

	// Total number of cards to generate for this settings group
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Cards", meta=(ClampMin="1", ClampMax="100000"))
	int32 TargetNumberOfCards = 2000;

	// Target number of triangles of the final mesh, only for adaptive subdivision
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Geometry", meta=(ClampMin="2", ClampMax="1000000"))
	int32 TargetTriangleCount = 20000;

	// Maximum number of cards to assign to flyaway strands
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Cards", meta=(ClampMin="0", ClampMax="1000"))
	int MaxNumberOfFlyaways = 10;

	// Generate multiple cards (3) per strand clump to give the appearance of volume
	UPROPERTY(BlueprintReadOnly, Category="Geometry")
	bool UseMultiCardClumps = true;

	// Use adaptive subdivision when generating card geometry
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Geometry|Advanced")
	bool UseAdaptiveSubdivision = true;

	// Maximum number of segments along each card (aligned with the strands), ignored for adaptive subdivision
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Geometry|Advanced", meta=(ClampMin="1", ClampMax="15"))
	int32 MaxVerticalSegmentsPerCard = 10;

	// Number of strand clump textures per atlas
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Textures", meta=(ClampMin="1", ClampMax="300"))
	int32 NumberOfTexturesInAtlas = 75;

	// Scaling factor mapping strand-width to pixel-size for coverage texture
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Texture Rendering", meta=(ClampMin="0.0", ClampMax="300.0"))
	float StrandWidthScalingFactor = 1.0f;

	// Compress textures along strand direction to save vertical redolution,
	// if strands are all moving in nearly the same direction
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Texture Rendering")
	bool UseOptimizedCompressionFactor = true;


	void Reset(TObjectPtr<UHairCardGeneratorGroupSettings> TemplateObject);
	void Reset(const TSharedPtr<FJsonObject>& TemplateSettings);
	void ResetFilterDefault(const TArray<FName>& CardGroupIds);
	void UpdateStrandFilter();
	void UpdateGenerateFilename(const FString& BaseFilename, int index);
	void UpdateParentName(const FString& BaseParentName, int index);

	void RemoveSettingsFilterGroup();

	//~ Begin UObject interface
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface

	uint8 GetFilterGroupGeneratedFlags() const;
	bool GetReduceFromPreviousLOD() const;

	void BuildDDCSubKey(FArchive& Ar);

	int32 GetStrandCount() {return StrandCount;}

	/** Get the filter name */
	const FName& GetFilterName() const {return FilterName;}

	/** Set the filter name with a given identifier */
	void SetFilterName(const FName& InFilterName) {FilterName = InFilterName;}

private:
	void SetSource(TObjectPtr<const UGroomAsset> SourceGroom, int32 GenLODIndex = 0);

	FString GetGeneratedFilterGroupSettingsFilename() const;

	void SerializeGroupSettings(FArchive& Ar);

	uint8 GetPipelineFilterGroupDifferences() const;
	void ClearPipelineGeneratedFilterGroup() const;
	void WritePipelineGeneratedFilterGroup();

	void SetStrandStats();
};
