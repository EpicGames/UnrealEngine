// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HairCardGenTypes.generated.h"

/**
 * Output of GenerateClumps / input to SetOptimizations.
 * Per-settings-group; saved and loaded by SaveClumpData / LoadClumpData.
 */
USTRUCT(BlueprintType)
struct FHairCardClumpData
{
	GENERATED_BODY()

	/** (N_strands,) clump label index per strand */
	UPROPERTY(BlueprintReadWrite, Category="Clustering")
	TArray<int32> StrandLabels;

	/** Largest main-clump index; used to separate flyaway from standard clumps in label list */
	UPROPERTY(BlueprintReadWrite, Category="Clustering")
	int32  MaxMainClump = 0;
};


/**
 * Card mesh geometry for one settings group, stored as flat concatenated arrays.
 * All cards are concatenated; use CardVertOffsets/CardFaceOffsets to slice per card.
 * Embedded in FHairCardGeomData. Replaces per-card OBJ files in the Dataflow path (Layer 5B).
 */
USTRUCT(BlueprintType)
struct FHairCardMeshData
{
	GENERATED_BODY()

	/** (TotalVerts * 3,) flattened xyz — all cards concatenated */
	UPROPERTY(BlueprintReadWrite, Category="Mesh")
	TArray<float> Verts;

	/** (TotalFaces * 3,) flattened vertex indices — card-local, use offsets to get global */
	UPROPERTY(BlueprintReadWrite, Category="Mesh")
	TArray<int32> Faces;

	/** (TotalVerts * 2,) flattened uv — card surface parameterisation used for strand projection */
	UPROPERTY(BlueprintReadWrite, Category="Mesh")
	TArray<float> SurfaceUvs;

	/** (N_cards,) start vert index per card into Verts/SurfaceUVs */
	UPROPERTY(BlueprintReadWrite, Category="Mesh")
	TArray<int32> CardVertOffsets;

	/** (N_cards,) vert count per card */
	UPROPERTY(BlueprintReadWrite, Category="Mesh")
	TArray<int32> CardVertCounts;

	/** (N_cards,) start face index per card into Faces */
	UPROPERTY(BlueprintReadWrite, Category="Mesh")
	TArray<int32> CardFaceOffsets;

	/** (N_cards,) face count per card */
	UPROPERTY(BlueprintReadWrite, Category="Mesh")
	TArray<int32> CardFaceCounts;
};


/**
 * Output of GenerateCardsGeometry / input to ClusterTextures and all cross-group stages.
 * Per-settings-group; saved and loaded by SaveGeomData / LoadGeomData.
 * CardCount is used by cross-group stages to compute global card ID offsets.
 */
USTRUCT(BlueprintType)
struct FHairCardGeomData
{
	GENERATED_BODY()

	/** (N_cards,) card length */
	UPROPERTY(BlueprintReadWrite, Category="Geometry")
	TArray<float> CardsLength;

	/** (N_cards,) card width */
	UPROPERTY(BlueprintReadWrite, Category="Geometry")
	TArray<float> CardsWidth;

	/** (N_cards,) left/right border ratio */
	UPROPERTY(BlueprintReadWrite, Category="Geometry")
	TArray<float> CardsLrRatio;

	/** (N_cards * 3,) flattened card top-vector (used for orientation) */
	UPROPERTY(BlueprintReadWrite, Category="Geometry")
	TArray<float> CardsTopVec;

	/**
	 * (N_cards,) card type id.
	 * Separates flyaway cards from standard cards for texture clustering.
	 * Stored in card_type.npy (formerly cards_texture_group.npy).
	 * 0 - standard card
	 * 1 - flyaway card
	 */
	UPROPERTY(BlueprintReadWrite, Category="Geometry")
	TArray<int32> CardsType;

	/** (N_strands,) card index per strand (global card ID in cross-group context) */
	UPROPERTY(BlueprintReadWrite, Category="Geometry")
	TArray<int32> CardsLabel;

	/** (N_cards,) physics group label per card */
	UPROPERTY(BlueprintReadWrite, Category="Geometry")
	TArray<uint8> CardsPhysGrpIdx;
};


/**
 * Output of ClusterTextures / input to all cross-group stages.
 * Per-settings-group; saved and loaded by SaveClusterData / LoadClusterData.
 */
USTRUCT(BlueprintType)
struct FHairCardTextureClusterData
{
	GENERATED_BODY()

	/** (N_cards,) cluster index per card */
	UPROPERTY(BlueprintReadWrite, Category="TextureCluster")
	TArray<int32> ClustersLabel;

	/** (N_clusters,) center card index per cluster */
	UPROPERTY(BlueprintReadWrite, Category="TextureCluster")
	TArray<int32> ClustersCenterId;

	/** (N_cards,) flip flag per card */
	UPROPERTY(BlueprintReadWrite, Category="TextureCluster")
	TArray<bool> CardsFlipped;
};


/**
 * Texture atlas layout information for each texture to render
 */
USTRUCT(BlueprintType)
struct FHairCardAtlasLayoutData
{
	GENERATED_BODY()

	/** (N_centers * 2,) flattened [width, height] texture size for each cluster center */
	UPROPERTY(BlueprintReadWrite, Category="TextureLayout")
	TArray<int32> TxtSize;

	/** (N_centers * 2,) flattened [x, y] atlas coordinate per cluster center */
	UPROPERTY(BlueprintReadWrite, Category="TextureLayout")
	TArray<int32> TxtCoords;

	/** (N_centers,) global card id per cluster center */
	UPROPERTY(BlueprintReadWrite, Category="TextureLayout")
	TArray<int32> CenterCardsId;

	// /** (N_centers * 2,) flattened [length, width] card size per cluster center */
	// UPROPERTY()
	// TArray<float> CenterCardsSize;
};


/**
 * Binary partition info used internally for current atlas layout algorithm
 */
USTRUCT(BlueprintType)
struct FHairCardAtlasBinpackData
{
	GENERATED_BODY()

	/** (N_nodes * 2,) flattened child node indices; -1 = None (leaf) */
	UPROPERTY(BlueprintReadWrite, Category="Binpack")
	TArray<int32> Children;

	/** (N_nodes * 4,) flattened [min_x, min_y, max_x, max_y] bounds per node */
	UPROPERTY(BlueprintReadWrite, Category="Binpack")
	TArray<int32> Bounds;

	/** (N_nodes,) node full flag */
	UPROPERTY(BlueprintReadWrite, Category="Binpack")
	TArray<bool> IsFull;

	/** (N_nodes,) node reserved flag */
	UPROPERTY(BlueprintReadWrite, Category="Binpack")
	TArray<bool> IsReserved;

	/** Margin in pixels; invariant across all nodes */
	UPROPERTY(BlueprintReadWrite, Category="Binpack")
	int32 MarginPx = 0;
};


/**
 * Per-vertex atlas UV coordinates, produced by GenerateTextureLayout alongside the layout data.
 * Flat array parallel to FHairCardMeshData.SurfaceUvs — same vertex ordering, no sentinels.
 */
USTRUCT(BlueprintType)
struct FHairCardAtlasUVData
{
	GENERATED_BODY()

	/** (TotalVerts * 2,) flattened [u, v] atlas UV per vertex */
	UPROPERTY(BlueprintReadWrite, Category="TextureLayout")
	TArray<float> VertexUvs;
};
