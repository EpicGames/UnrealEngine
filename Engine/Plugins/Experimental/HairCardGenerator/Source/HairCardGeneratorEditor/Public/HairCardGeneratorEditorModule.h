// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IHairCardGeneratorEditor.h"
#include "HairCardGenTypes.h"
#include <functional>

struct FHairGroupsCardsSourceDescription;
class UGroomAsset;
class UHairCardGeneratorPluginSettings;
class UHairCardGenControllerBase;

struct HAIRCARDGENERATOREDITOR_API FCardsGenerationAdvancedOptions
{
	// Use previous LOD generated cards and textures but reduce triangle count and flyaways
	bool bReduceCardsFromPreviousLOD = false;

	// Generate geometry for all groom groups on group 0
	bool bGenerateGeometryForAllGroups = true;

	// Seed value for pseudo-random number generation (set to a specific value for repeatable results)
	int32 RandomSeed = 0;

	// Place new card textures in reserved space from previous LOD
	bool bUseReservedSpaceFromPreviousLOD = false;

	// Size of hair card texture atlases
	uint8 AtlasSize = 12;

	// Percentage of texture atlas space to reserve for higher LODs 
	int32 ReserveTextureSpaceLOD = 0;

	// Use strand width
	bool bUseGroomAssetStrandWidth = true;
	
	// Default directory to be used if defined
	FString DefaultDirectory = "";
	
	// Default name to be used if defined
	FString DefaultName = "";
};

/** FHairCardGeneratorUtils has static interface functions to be used in other modules */
struct HAIRCARDGENERATOREDITOR_API FHairCardGeneratorUtils
{
	using FGeneratorFunction = std::function<bool(const TObjectPtr<const UHairCardGeneratorPluginSettings>&, const int32, const uint8)>;
	using FBuilderFunction = std::function<void(TArray<TArray<FVector>>&)>;
	
	/** Build the generation settings for a given grom asset and a cards description */
	static bool BuildGenerationSettings(const bool bQuerySettings, UGroomAsset* GroomAsset, FHairGroupsCardsSourceDescription& CardsDesc,
		TObjectPtr<UHairCardGeneratorPluginSettings>& GenerationSettings, uint8& GenerationFlags, uint8& PipelineFlags, const FCardsGenerationAdvancedOptions& AdvancedOptions);

	/** Load the generation settings for a given grom asset and a cards description */
	static bool LoadGenerationSettings(const TObjectPtr<UHairCardGeneratorPluginSettings>& GenerationSettings);

	/** Generate the cards clumps based on the settings given a filter index */
	static bool GenerateCardsClumps(const TObjectPtr<const UHairCardGeneratorPluginSettings>& GenerationSettings, const int32 FilterIndex, const uint8 GenFlags, FHairCardClumpData& OutClumpData);

	/** Generate the cards geometry based on the settings given a filter index */
	static bool GenerateCardsGeometry(const TObjectPtr<const UHairCardGeneratorPluginSettings>& GenerationSettings, const int32 FilterIndex, const uint8 GenFlags, FHairCardGeomData& OutGeomData, FHairCardMeshData& OutMeshData, int32& OutCardCount);

	/** Generate the cards texture clusters based on the settings given a filter index */
	static bool GenerateCardsTexturesClusters(const TObjectPtr<const UHairCardGeneratorPluginSettings>& GenerationSettings, const int32 FilterIndex, const uint8 GenFlags, FHairCardTextureClusterData& OutClusterData);

	/** Generate the cards texture layout and atlases. OutLayoutData is populated from layout generation or loaded from disk.
	 *  OutAtlasUVData is populated when layout is generated; empty when layout is skipped (only rendering reruns). */
	static bool GenerateTexturesLayoutAndAtlases(const TObjectPtr<const UHairCardGeneratorPluginSettings>& GenerationSettings, const uint8 GenFlags, FHairCardAtlasLayoutData& OutLayoutData, FHairCardAtlasUVData& OutAtlasUVData);
	
	/** Run the cards generation function */
	static bool RunCardsGeneration(const TObjectPtr<UHairCardGeneratorPluginSettings>& GenerationSettings, const uint8 PipelineFlags, const FGeneratorFunction& PipelineFunction, const bool bCheckFlags = true);

	/** Build and save the cards assets to disk */
	static bool BuildCardsAssets(UGroomAsset* NewGroomAsset, FHairGroupsCardsSourceDescription& CardsDesc,
		const TObjectPtr<UHairCardGeneratorPluginSettings>& GenerationSettings, const uint8 GenFlags);

	/** Load the groom strands */
	static bool LoadGroomStrands(const UGroomAsset* GroomAsset, const FBuilderFunction& BuilderFunction);
	
	/** Build the cards group names */
	static void BuildCardsGroups(TObjectPtr<const UGroomAsset> GroomAsset, TArray<FName>& GroupNames, TArray<int32>& GroupCount);
};

/** FHairCardGeneratorEditorModule  */
class FHairCardGeneratorEditorModule : public IHairCardGeneratorEditor
{
public:
	//~ Begin IModuleInterface API
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface API

	//~ Begin IHairCardGenerator API
	virtual bool GenerateHairCardsForLOD(UGroomAsset* Groom, FHairGroupsCardsSourceDescription& CardsDesc) override;
	virtual bool IsCompatibleSettings(UHairCardGenerationSettings* OldSettings) override;
	//~ End IHairCardGenerator API
};

