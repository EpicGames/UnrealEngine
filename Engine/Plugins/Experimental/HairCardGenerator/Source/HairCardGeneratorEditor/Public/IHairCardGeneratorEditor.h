// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "IHairCardGenerator.h"

class IHairCardGeneratorEditor : public IModuleInterface, public IHairCardGenerator
{
public:
	static inline IHairCardGeneratorEditor& Get()
	{
		return FModuleManager::LoadModuleChecked<IHairCardGeneratorEditor>("HairCardGeneratorEditor");
	}
};
